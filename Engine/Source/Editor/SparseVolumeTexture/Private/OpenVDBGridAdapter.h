// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "SparseVolumeTextureOpenVDB.h"

#if OPENVDB_AVAILABLE

#include "CoreMinimal.h"

template<typename ValueType>
constexpr uint32 GetOpenVDBValueNumComponents()
{
	return (uint32)ValueType::size;
}

template<>
constexpr uint32 GetOpenVDBValueNumComponents<FOpenVDBHalf>()
{
	return 1;
}

template<>
constexpr uint32 GetOpenVDBValueNumComponents<float>()
{
	return 1;
}

template<>
constexpr uint32 GetOpenVDBValueNumComponents<double>()
{
	return 1;
}

template<typename ValueType>
float GetOpenVDBValueComponent(const ValueType& Value, uint32 ComponentIndex)
{
	return (float)Value[FMath::Min(ComponentIndex, (uint32)ValueType::size)];
}

template<>
inline float GetOpenVDBValueComponent<FOpenVDBHalf>(const FOpenVDBHalf& Value, uint32 ComponentIndex)
{
	return (float)Value;
}

template<>
inline float GetOpenVDBValueComponent<float>(const float& Value, uint32 ComponentIndex)
{
	return Value;
}

template<>
inline float GetOpenVDBValueComponent<double>(const double& Value, uint32 ComponentIndex)
{
	return (float)Value;
}

template<typename ValueType>
ValueType GetOpenVDBMaxComponent(const ValueType& ValueA, const ValueType& ValueB)
{
	return openvdb::math::maxComponent(ValueA, ValueB);
}

template<>
inline FOpenVDBHalf GetOpenVDBMaxComponent<FOpenVDBHalf>(const FOpenVDBHalf& ValueA, const FOpenVDBHalf& ValueB)
{
	return FOpenVDBHalf(FMath::Max((float)ValueA, (float)ValueB));
}

template<>
inline float GetOpenVDBMaxComponent<float>(const float& ValueA, const float& ValueB)
{
	return FMath::Max(ValueA, ValueB);
}

template<>
inline double GetOpenVDBMaxComponent<double>(const double& ValueA, const double& ValueB)
{
	return FMath::Max(ValueA, ValueB);
}

template<typename ValueType>
ValueType GetOpenVDBMinComponent(const ValueType& ValueA, const ValueType& ValueB)
{
	return openvdb::math::minComponent(ValueA, ValueB);
}

template<>
inline FOpenVDBHalf GetOpenVDBMinComponent<FOpenVDBHalf>(const FOpenVDBHalf& ValueA, const FOpenVDBHalf& ValueB)
{
	return FOpenVDBHalf(FMath::Min((float)ValueA, (float)ValueB));
}

template<>
inline float GetOpenVDBMinComponent<float>(const float& ValueA, const float& ValueB)
{
	return FMath::Min(ValueA, ValueB);
}

template<>
inline double GetOpenVDBMinComponent<double>(const double& ValueA, const double& ValueB)
{
	return FMath::Min(ValueA, ValueB);
}

// Interface for sampling and getting other values from an openVDB grid, abstracting away the actual type/format of the underlying grid.
class IOpenVDBGridAdapterBase
{
public:
	virtual void IteratePhysical(TFunctionRef<void(const FIntVector3& Coord, uint32 NumComponents, float* VoxelValues)> OnVisit) const = 0;
	virtual float Sample(const FIntVector3& VoxelCoord, uint32 ComponentIndex) const = 0;
	virtual void GetMinMaxValue(uint32 ComponentIndex, float* OutMinVal, float* OutMaxVal) const = 0;
	virtual float GetBackgroundValue(uint32 ComponentIndex) const = 0;
	virtual ~IOpenVDBGridAdapterBase() = default;
};

template<typename GridType>
class TOpenVDBGridAdapter : public IOpenVDBGridAdapterBase
{
	using ValueType = typename GridType::ValueType;
public:
	explicit TOpenVDBGridAdapter(openvdb::SharedPtr<GridType> Grid)
		: Grid(Grid), Accessor(Grid->getConstAccessor())
	{
	}

	void IteratePhysical(TFunctionRef<void(const FIntVector3& Coord, uint32 NumComponents, float* VoxelValues)> OnVisit) const override
	{
		constexpr uint32 NumComponents = GetOpenVDBValueNumComponents<ValueType>();
		static_assert(NumComponents <= 4);

		// Iterate over both voxels and tiles.
		for (auto ValueIt = Grid->cbeginValueOn(); ValueIt; ++ValueIt)
		{
			const ValueType VoxelValue = ValueIt.getValue();
			const openvdb::Coord CoordVDB = ValueIt.getCoord();
			
			float VoxelValueComponents[4]{};
			for (uint32 ComponentIdx = 0; ComponentIdx < NumComponents; ++ComponentIdx)
			{
				VoxelValueComponents[ComponentIdx] = GetOpenVDBValueComponent(VoxelValue, ComponentIdx);
			}
			const FIntVector3 Coord(CoordVDB[0], CoordVDB[1], CoordVDB[2]);

			if (ValueIt.isVoxelValue())
			{
				// A single voxel
				OnVisit(Coord, NumComponents, VoxelValueComponents);
			}
			else
			{
				// Tile, a higher level node specifying a single value for all finer level nodes/children without actually allocating memory for them.
				openvdb::CoordBBox TileAABB;
				ValueIt.getBoundingBox(TileAABB);
				check(TileAABB.min().x() == Coord.X && TileAABB.min().y() == Coord.Y && TileAABB.min().z() == Coord.Z);
				const openvdb::Coord AABBDimVDB = TileAABB.dim();
				const FIntVector3 AABBDim = FIntVector3(AABBDimVDB.x(), AABBDimVDB.y(), AABBDimVDB.z());
				
				// Iterate over tile AABB and invoke callback for every (virtual) voxel.
				for (int32 Z = 0; Z < AABBDim.Z; ++Z)
				{
					for (int32 Y = 0; Y < AABBDim.Y; ++Y)
					{
						for (int32 X = 0; X < AABBDim.X; ++X)
						{
							const FIntVector3 TileLocalCoord(Coord.X + X, Coord.Y + Y, Coord.Z + Z);
							OnVisit(TileLocalCoord, NumComponents, VoxelValueComponents);
						}
					}
				}
			}
		}
	}

	float Sample(const FIntVector3& VoxelCoord, uint32 ComponentIndex) const override
	{
		ValueType VoxelValue = Accessor.getValue(openvdb::Coord(VoxelCoord.X, VoxelCoord.Y, VoxelCoord.Z));
		return GetOpenVDBValueComponent(VoxelValue, ComponentIndex);
	}

	void GetMinMaxValue(uint32 ComponentIndex, float* OutMinVal, float* OutMaxVal) const override
	{
		*OutMinVal = FLT_MAX;
		*OutMaxVal = -FLT_MAX;
		if (auto Iter = Grid->tree().cbeginValueOn())
		{
			ValueType MinVal = *Iter;
			ValueType MaxVal = MinVal;

			for (++Iter; Iter; ++Iter)
			{
				ValueType Value = *Iter;
				MinVal = GetOpenVDBMinComponent(MinVal, Value);
				MaxVal = GetOpenVDBMaxComponent(MaxVal, Value);
			}

			*OutMinVal = GetOpenVDBValueComponent(MinVal, ComponentIndex);
			*OutMaxVal = GetOpenVDBValueComponent(MaxVal, ComponentIndex);
		}
	}

	float GetBackgroundValue(uint32 ComponentIndex) const override
	{
		return GetOpenVDBValueComponent(Grid->background(), ComponentIndex);
	}

private:
	typename GridType::Ptr Grid;
	typename GridType::ConstAccessor Accessor;
};

// This is just to not have to repeat the GridType multiple times per MakeShared<>() call.
template<typename GridType>
TSharedPtr<IOpenVDBGridAdapterBase> CreateOpenVDBGridAdapterInternal(openvdb::GridBase::Ptr Grid)
{
	return MakeShared<TOpenVDBGridAdapter<GridType>>(TOpenVDBGridAdapter<GridType>(openvdb::gridPtrCast<GridType>(Grid)));
}

// Creates an adapter suitable for the type of the given grid or nullptr if the type is not supported.
inline TSharedPtr<IOpenVDBGridAdapterBase> CreateOpenVDBGridAdapter(openvdb::GridBase::Ptr Grid)
{
	if (Grid->isType<FOpenVDBHalf1Grid>())
	{
		return CreateOpenVDBGridAdapterInternal<FOpenVDBHalf1Grid>(Grid);
	}
	else if (Grid->isType<FOpenVDBHalf2Grid>())
	{
		return CreateOpenVDBGridAdapterInternal<FOpenVDBHalf2Grid>(Grid);
	}
	else if (Grid->isType<FOpenVDBHalf3Grid>())
	{
		return CreateOpenVDBGridAdapterInternal<FOpenVDBHalf3Grid>(Grid);
	}
	else if (Grid->isType<FOpenVDBHalf4Grid>())
	{
		return CreateOpenVDBGridAdapterInternal<FOpenVDBHalf4Grid>(Grid);
	}
	else if (Grid->isType<FOpenVDBFloat1Grid>())
	{
		return CreateOpenVDBGridAdapterInternal<FOpenVDBFloat1Grid>(Grid);
	}
	else if (Grid->isType<FOpenVDBFloat2Grid>())
	{
		return CreateOpenVDBGridAdapterInternal<FOpenVDBFloat2Grid>(Grid);
	}
	else if (Grid->isType<FOpenVDBFloat3Grid>())
	{
		return CreateOpenVDBGridAdapterInternal<FOpenVDBFloat3Grid>(Grid);
	}
	else if (Grid->isType<FOpenVDBFloat4Grid>())
	{
		return CreateOpenVDBGridAdapterInternal<FOpenVDBFloat4Grid>(Grid);
	}
	else if (Grid->isType<FOpenVDBDouble1Grid>())
	{
		return CreateOpenVDBGridAdapterInternal<FOpenVDBDouble1Grid>(Grid);
	}
	else if (Grid->isType<FOpenVDBDouble2Grid>())
	{
		return CreateOpenVDBGridAdapterInternal<FOpenVDBDouble2Grid>(Grid);
	}
	else if (Grid->isType<FOpenVDBDouble3Grid>())
	{
		return CreateOpenVDBGridAdapterInternal<FOpenVDBDouble3Grid>(Grid);
	}
	else if (Grid->isType<FOpenVDBDouble4Grid>())
	{
		return CreateOpenVDBGridAdapterInternal<FOpenVDBDouble4Grid>(Grid);
	}
	return nullptr;
}

#endif //OPENVDB_AVAILABLE
#endif // WITH_EDITOR
