// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenVDBImportWindow.h"

#include "Widgets/Input/SButton.h"
#include "SPrimaryButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#include "IDocumentation.h"
#include "Editor.h"
#include "SparseVolumeTextureOpenVDBUtility.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "OpenVDBImportOptions.h"

#define LOCTEXT_NAMESPACE "SOpenVDBImportWindow"

#define VDB_GRID_ROW_NAME_GRID_INDEX TEXT("GridIndex")
#define VDB_GRID_ROW_NAME_GRID_TYPE TEXT("GridType")
#define VDB_GRID_ROW_NAME_GRID_NAME TEXT("GridName")
#define VDB_GRID_ROW_NAME_GRID_DIMS TEXT("GridDims")

static FText GetGridComboBoxItemText(TSharedPtr<FOpenVDBGridComponentInfo> InItem)
{
	return InItem ? FText::FromString(InItem->DisplayString) : LOCTEXT("NoneGrid", "<None>");
};

static FText GetFormatComboBoxItemText(TSharedPtr<ESparseVolumeAttributesFormat> InItem)
{
	const TCHAR* FormatStr = TEXT("<None>");
	if (InItem)
	{
		switch (*InItem)
		{
		case ESparseVolumeAttributesFormat::Unorm8: FormatStr = TEXT("8bit unorm"); break;
		case ESparseVolumeAttributesFormat::Float16: FormatStr = TEXT("16bit float"); break;
		case ESparseVolumeAttributesFormat::Float32: FormatStr = TEXT("32bit float"); break;
		}
	}
	return FText::FromString(FormatStr);
}

void SOpenVDBImportWindow::Construct(const FArguments& InArgs)
{
	ImportOptions = InArgs._ImportOptions;
	DefaultImportOptions = InArgs._DefaultImportOptions;
	bIsSequence = InArgs._NumFoundFiles > 1;
	OpenVDBGridInfo = InArgs._OpenVDBGridInfo;
	OpenVDBGridComponentInfo = InArgs._OpenVDBGridComponentInfo;
	OpenVDBSupportedTargetFormats = InArgs._OpenVDBSupportedTargetFormats;
	WidgetWindow = InArgs._WidgetWindow;

	TSharedPtr<SBox> ImportTypeDisplay;
	TSharedPtr<SHorizontalBox> OpenVDBHeaderButtons;
	TSharedPtr<SBox> InspectorBox;
	this->ChildSlot
	[
		SNew(SBox)
		.MaxDesiredHeight(InArgs._MaxWindowHeight)
		.MaxDesiredWidth(InArgs._MaxWindowWidth)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SAssignNew(ImportTypeDisplay, SBox)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SNew(SBorder)
				.Padding(FMargin(3))
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("Import_CurrentFileTitle", "Current Asset: "))
					]
					+ SHorizontalBox::Slot()
					.Padding(5, 0, 0, 0)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(InArgs._FullPath)
						.ToolTipText(InArgs._FullPath)
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SAssignNew(InspectorBox, SBox)
				.MaxDesiredHeight(650.0f)
				.WidthOverride(400.0f)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(2)
				+ SUniformGridPanel::Slot(1, 0)
				[
					SAssignNew(ImportButton, SPrimaryButton)
					.Text(LOCTEXT("OpenVDBImportWindow_Import", "Import"))
					.IsEnabled(this, &SOpenVDBImportWindow::CanImport)
					.OnClicked(this, &SOpenVDBImportWindow::OnImport)
				]
				+ SUniformGridPanel::Slot(2, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("OpenVDBImportWindow_Cancel", "Cancel"))
					.ToolTipText(LOCTEXT("OpenVDBImportWindow_Cancel_ToolTip", "Cancels importing this OpenVDB file"))
					.OnClicked(this, &SOpenVDBImportWindow::OnCancel)
				]
			]
		]
	];

	InspectorBox->SetContent(
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.AutoWidth()
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ImportAsSequenceCheckBoxLabel", "Import Sequence"))
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.AutoWidth()
			.Padding(2.0f)
			[
				SAssignNew(ImportAsSequenceCheckBox, SCheckBox)
				.ToolTipText(LOCTEXT("ImportAsSequenceCheckBoxTooltip", "Import multiple sequentially labeled .vdb files as a single animated SparseVirtualTexture sequence."))
				.IsChecked(bIsSequence)
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.AutoWidth()
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(FString::Format(TEXT("Found {0} File(s)"), { InArgs._NumFoundFiles })))
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SAssignNew(AttributesAConfigurator, SOpenVDBAttributesConfigurator)
			.AttributesDesc(&ImportOptions->Attributes[0])
			.OpenVDBGridComponentInfo(OpenVDBGridComponentInfo)
			.OpenVDBSupportedTargetFormats(OpenVDBSupportedTargetFormats)
			.AttributesName(LOCTEXT("OpenVDBImportWindow_AttributesA", "Attributes A"))
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SAssignNew(AttributesBConfigurator, SOpenVDBAttributesConfigurator)
			.AttributesDesc(&ImportOptions->Attributes[1])
			.OpenVDBGridComponentInfo(OpenVDBGridComponentInfo)
			.OpenVDBSupportedTargetFormats(OpenVDBSupportedTargetFormats)
			.AttributesName(LOCTEXT("OpenVDBImportWindow_AttributesB", "Attributes B"))
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("OpenVDBImportWindow_FileInfo", "Source File Grid Info"))
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SNew(SBorder)
			.Padding(FMargin(3))
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SListView<TSharedPtr<FOpenVDBGridInfo>>)
				.ScrollbarVisibility(EVisibility::Visible)
				.ListItemsSource(OpenVDBGridInfo)
				.OnGenerateRow(this, &SOpenVDBImportWindow::GenerateGridInfoItemRow)
				.HeaderRow
				(
					SNew(SHeaderRow)
					+ SHeaderRow::Column(VDB_GRID_ROW_NAME_GRID_INDEX).DefaultLabel(LOCTEXT("GridIndex", "Index")).FillWidth(0.05f)
					+ SHeaderRow::Column(VDB_GRID_ROW_NAME_GRID_NAME).DefaultLabel(LOCTEXT("GridName", "Name")).FillWidth(0.15f)
					+ SHeaderRow::Column(VDB_GRID_ROW_NAME_GRID_TYPE).DefaultLabel(LOCTEXT("GridType", "Type")).FillWidth(0.1f)
					+ SHeaderRow::Column(VDB_GRID_ROW_NAME_GRID_DIMS).DefaultLabel(LOCTEXT("GridDims", "Dimensions")).FillWidth(0.25f)
				)
			]
		]
	);

	SetDefaultGridAssignment();

	ImportTypeDisplay->SetContent(
		SNew(SBorder)
		.Padding(FMargin(3))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SOpenVDBImportWindow::GetImportTypeDisplayText)
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				IDocumentation::Get()->CreateAnchor(FString("sparse-volume-textures-in-unreal-engine"))
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			[
				SAssignNew(OpenVDBHeaderButtons, SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(2.0f, 0.0f))
				[
					SNew(SButton)
					.Text(LOCTEXT("OpenVDBImportWindow_ResetOptions", "Reset to Default"))
					.OnClicked(this, &SOpenVDBImportWindow::OnResetToDefaultClick)
				]
			]
		]
	);

	RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SOpenVDBImportWindow::SetFocusPostConstruct));
}

FReply SOpenVDBImportWindow::OnImport()
{
	bShouldImport = true;
	if (WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

FReply SOpenVDBImportWindow::OnCancel()
{
	bShouldImport = false;
	if (WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

bool SOpenVDBImportWindow::ShouldImport() const
{
	return bShouldImport;
}

bool SOpenVDBImportWindow::ShouldImportAsSequence() const
{
	return ImportAsSequenceCheckBox->IsChecked();
}

EActiveTimerReturnType SOpenVDBImportWindow::SetFocusPostConstruct(double InCurrentTime, float InDeltaTime)
{
	if (ImportButton.IsValid())
	{
		FSlateApplication::Get().SetKeyboardFocus(ImportButton, EFocusCause::SetDirectly);
	}

	return EActiveTimerReturnType::Stop;
}

TSharedRef<ITableRow> SOpenVDBImportWindow::GenerateGridInfoItemRow(TSharedPtr<FOpenVDBGridInfo> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SOpenVDBGridInfoTableRow, OwnerTable).OpenVDBGridInfo(Item);
}

bool SOpenVDBImportWindow::CanImport() const
{
	for (const FOpenVDBSparseVolumeAttributesDesc& AttributesDesc : ImportOptions->Attributes)
	{
		for (const FOpenVDBSparseVolumeComponentMapping& Mapping : AttributesDesc.Mappings)
		{
			if (Mapping.SourceGridIndex != INDEX_NONE && Mapping.SourceComponentIndex != INDEX_NONE)
			{
				return true;
			}
		}
	}
	return false;
}

FReply SOpenVDBImportWindow::OnResetToDefaultClick()
{
	SetDefaultGridAssignment();
	return FReply::Handled();
}

FText SOpenVDBImportWindow::GetImportTypeDisplayText() const
{
	return ShouldImportAsSequence() ? LOCTEXT("OpenVDBImportWindow_ImportTypeAnimated", "Import OpenVDB animation") : LOCTEXT("OpenVDBImportWindow_ImportTypeStatic", "Import static OpenVDB");
}

void SOpenVDBImportWindow::SetDefaultGridAssignment()
{
	check(OpenVDBGridComponentInfo);

	*ImportOptions = *DefaultImportOptions;

	ImportAsSequenceCheckBox->SetIsChecked(bIsSequence ? ECheckBoxState::Checked : ECheckBoxState::Unchecked);
	AttributesAConfigurator->RefreshUIFromData();
	AttributesBConfigurator->RefreshUIFromData();
}

void SOpenVDBComponentPicker::Construct(const FArguments& InArgs)
{
	AttributesDesc = InArgs._AttributesDesc;
	ComponentIndex = InArgs._ComponentIndex;
	OpenVDBGridComponentInfo = InArgs._OpenVDBGridComponentInfo;
	
	check(ComponentIndex < 4);
	const TCHAR* ComponentLabels[] = { TEXT("R"), TEXT("G"), TEXT("B"), TEXT("A") };

	this->ChildSlot
		[
			SNew(SHorizontalBox)
			
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(1.0f)
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(ComponentLabels[ComponentIndex]))
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(2.0f)
			[
				SNew(SBox)
				.WidthOverride(300.0f)
				[
					SAssignNew(GridComboBox, SComboBox<TSharedPtr<FOpenVDBGridComponentInfo>>)
					.OptionsSource(OpenVDBGridComponentInfo)
					.OnGenerateWidget_Lambda([](TSharedPtr<FOpenVDBGridComponentInfo> InItem)
					{
						return SNew(STextBlock)
						.Text(GetGridComboBoxItemText(InItem));
					})
					.OnSelectionChanged_Lambda([this](TSharedPtr<FOpenVDBGridComponentInfo> InItem, ESelectInfo::Type)
					{
						AttributesDesc->Mappings[ComponentIndex].SourceGridIndex = InItem ? InItem->Index : INDEX_NONE;
						AttributesDesc->Mappings[ComponentIndex].SourceComponentIndex = InItem ? InItem->ComponentIndex : INDEX_NONE;
					})
					[
						SNew(STextBlock)
						.Text_Lambda([this]()
						{
							return GetGridComboBoxItemText(GridComboBox->GetSelectedItem());
						})
					]
				]
			]
		];
}

void SOpenVDBComponentPicker::RefreshUIFromData()
{
	for (const TSharedPtr<FOpenVDBGridComponentInfo>& Grid : *OpenVDBGridComponentInfo)
	{
		if (Grid->Index == AttributesDesc->Mappings[ComponentIndex].SourceGridIndex && Grid->ComponentIndex == AttributesDesc->Mappings[ComponentIndex].SourceComponentIndex)
		{
			GridComboBox->SetSelectedItem(Grid);
			break;
		}
	}
}

void SOpenVDBAttributesConfigurator::Construct(const FArguments& InArgs)
{
	AttributesDesc = InArgs._AttributesDesc;
	OpenVDBSupportedTargetFormats = InArgs._OpenVDBSupportedTargetFormats;

	for (uint32 ComponentIndex = 0; ComponentIndex < 4; ++ComponentIndex)
	{
		ComponentPickers[ComponentIndex] =
			SNew(SOpenVDBComponentPicker)
			.AttributesDesc(AttributesDesc)
			.ComponentIndex(ComponentIndex)
			.OpenVDBGridComponentInfo(InArgs._OpenVDBGridComponentInfo);
	}

	this->ChildSlot
		[
			SNew(SVerticalBox)
			
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.FillWidth(1.0f)
				.Padding(2.0f)
				[
					SNew(STextBlock)
					.Text(InArgs._AttributesName)
				]

				+ SHorizontalBox::Slot()
				.FillWidth(0.5f)
				.VAlign(VAlign_Center)
				.Padding(2.0f)
				[
					SNew(SBox)
					.WidthOverride(50.0f)
					[
						SAssignNew(FormatComboBox, SComboBox<TSharedPtr<ESparseVolumeAttributesFormat>>)
						.OptionsSource(OpenVDBSupportedTargetFormats)
						.OnGenerateWidget_Lambda([](TSharedPtr<ESparseVolumeAttributesFormat> InItem)
						{
							return SNew(STextBlock)
							.Text(GetFormatComboBoxItemText(InItem));
						})
						.OnSelectionChanged_Lambda([this](TSharedPtr<ESparseVolumeAttributesFormat> InItem, ESelectInfo::Type)
						{
							AttributesDesc->Format = InItem ? *InItem : ESparseVolumeAttributesFormat::Float32;
						})
						[
							SNew(STextBlock)
							.Text_Lambda([this]()
							{
								return GetFormatComboBoxItemText(FormatComboBox->GetSelectedItem());
							})
						]
					]
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SNew(SBorder)
				.Padding(FMargin(3))
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2)
					[
						ComponentPickers[0]->AsShared()
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2)
					[
						ComponentPickers[1]->AsShared()
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2)
					[
						ComponentPickers[2]->AsShared()
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2)
					[
						ComponentPickers[3]->AsShared()
					]
				]
			]
		];
}

void SOpenVDBAttributesConfigurator::RefreshUIFromData()
{
	for (auto& Format : *OpenVDBSupportedTargetFormats)
	{
		if (*Format == AttributesDesc->Format)
		{
			FormatComboBox->SetSelectedItem(Format);
			break;
		}
	}
	for (uint32 i = 0; i < 4; ++i)
	{
		ComponentPickers[i]->RefreshUIFromData();
	}
}

void SOpenVDBGridInfoTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView)
{
	OpenVDBGridInfo = InArgs._OpenVDBGridInfo;
	SMultiColumnTableRow<TSharedPtr<FOpenVDBGridInfo>>::Construct(FSuperRowType::FArguments(), OwnerTableView);
}

TSharedRef<SWidget> SOpenVDBGridInfoTableRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == VDB_GRID_ROW_NAME_GRID_INDEX)
	{
		return SNew(SBox).Padding(2).VAlign(VAlign_Center)
			[
				SNew(STextBlock).Text(FText::FromString(FString::Format(TEXT("{0}."), { OpenVDBGridInfo->Index })))
			];
	}
	else if (ColumnName == VDB_GRID_ROW_NAME_GRID_TYPE)
	{
		return SNew(SBox).Padding(2).VAlign(VAlign_Center)
			[
				SNew(STextBlock).Text(FText::FromString(OpenVDBGridTypeToString(OpenVDBGridInfo->Type)))
			];
	}
	else if (ColumnName == VDB_GRID_ROW_NAME_GRID_NAME)
	{
		return SNew(SBox).Padding(2).VAlign(VAlign_Center)
			[
				SNew(STextBlock).Text(FText::FromString(OpenVDBGridInfo->Name))
			];
	}
	else if (ColumnName == VDB_GRID_ROW_NAME_GRID_DIMS)
	{
		return SNew(SBox).Padding(2).VAlign(VAlign_Center)
			[
				SNew(STextBlock).Text(FText::FromString(OpenVDBGridInfo->VolumeActiveDim.ToString()))
			];
	}
	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE