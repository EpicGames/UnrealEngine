// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SCheckBox.h"

#include "SparseVolumeTexture/SparseVolumeTexture.h"

struct FOpenVDBGridInfo;
struct FOpenVDBImportOptions;
struct FOpenVDBGridComponentInfo;
struct FOpenVDBSparseVolumeAttributesDesc;
enum class ESparseVolumeAttributesFormat : uint8;

class SOpenVDBGridInfoTableRow : public SMultiColumnTableRow<TSharedPtr<FOpenVDBGridInfo>>
{
public:
	SLATE_BEGIN_ARGS(SOpenVDBGridInfoTableRow) {}
		SLATE_ARGUMENT(TSharedPtr<FOpenVDBGridInfo>, OpenVDBGridInfo)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView);
	TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:
	TSharedPtr<FOpenVDBGridInfo> OpenVDBGridInfo;
};

class SOpenVDBComponentPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SOpenVDBComponentPicker) {}
		SLATE_ARGUMENT(FOpenVDBSparseVolumeAttributesDesc*, AttributesDesc)
		SLATE_ARGUMENT(uint32, ComponentIndex)
		SLATE_ARGUMENT(const TArray<TSharedPtr<FOpenVDBGridComponentInfo>>*, OpenVDBGridComponentInfo)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	void RefreshUIFromData();

private:
	FOpenVDBSparseVolumeAttributesDesc* AttributesDesc;
	uint32 ComponentIndex;
	const TArray<TSharedPtr<FOpenVDBGridComponentInfo>>* OpenVDBGridComponentInfo;
	TSharedPtr<SComboBox<TSharedPtr<FOpenVDBGridComponentInfo>>> GridComboBox;
};

class SOpenVDBAttributesConfigurator : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SOpenVDBAttributesConfigurator) {}
		SLATE_ARGUMENT(FOpenVDBSparseVolumeAttributesDesc*, AttributesDesc)
		SLATE_ARGUMENT(const TArray<TSharedPtr<FOpenVDBGridComponentInfo>>*, OpenVDBGridComponentInfo)
		SLATE_ARGUMENT(const TArray<TSharedPtr<ESparseVolumeAttributesFormat>>*, OpenVDBSupportedTargetFormats)
		SLATE_ARGUMENT(FText, AttributesName)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	void RefreshUIFromData();

private:
	FOpenVDBSparseVolumeAttributesDesc* AttributesDesc;
	TSharedPtr<SOpenVDBComponentPicker> ComponentPickers[4];
	const TArray<TSharedPtr<ESparseVolumeAttributesFormat>>* OpenVDBSupportedTargetFormats;
	TSharedPtr<SComboBox<TSharedPtr<ESparseVolumeAttributesFormat>>> FormatComboBox;
};

class SOpenVDBImportWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SOpenVDBImportWindow) {}
		SLATE_ARGUMENT(FOpenVDBImportOptions*, ImportOptions)
		SLATE_ARGUMENT(const FOpenVDBImportOptions*, DefaultImportOptions)
		SLATE_ARGUMENT(int32, NumFoundFiles)
		SLATE_ARGUMENT(const TArray<TSharedPtr<FOpenVDBGridInfo>>*, OpenVDBGridInfo)
		SLATE_ARGUMENT(const TArray<TSharedPtr<FOpenVDBGridComponentInfo>>*, OpenVDBGridComponentInfo)
		SLATE_ARGUMENT(const TArray<TSharedPtr<ESparseVolumeAttributesFormat>>*, OpenVDBSupportedTargetFormats)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
		SLATE_ARGUMENT(FText, FullPath)
		SLATE_ARGUMENT(float, MaxWindowHeight)
		SLATE_ARGUMENT(float, MaxWindowWidth)
	SLATE_END_ARGS()

public:
	virtual bool SupportsKeyboardFocus() const override { return true; }
	void Construct(const FArguments& InArgs);
	FReply OnImport();
	FReply OnCancel();
	bool ShouldImport() const;
	bool ShouldImportAsSequence() const;

private:
	FOpenVDBImportOptions* ImportOptions;
	const FOpenVDBImportOptions* DefaultImportOptions;
	bool bIsSequence;
	const TArray<TSharedPtr<FOpenVDBGridInfo>>* OpenVDBGridInfo;
	const TArray<TSharedPtr<FOpenVDBGridComponentInfo>>* OpenVDBGridComponentInfo;
	const TArray<TSharedPtr<ESparseVolumeAttributesFormat>>* OpenVDBSupportedTargetFormats;
	TSharedPtr<SOpenVDBAttributesConfigurator> AttributesAConfigurator;
	TSharedPtr<SOpenVDBAttributesConfigurator> AttributesBConfigurator;
	TSharedPtr<SCheckBox> ImportAsSequenceCheckBox;
	TSharedPtr<SButton> ImportButton;
	TWeakPtr<SWindow> WidgetWindow;
	bool bShouldImport;

	EActiveTimerReturnType SetFocusPostConstruct(double InCurrentTime, float InDeltaTime);
	TSharedRef<ITableRow> GenerateGridInfoItemRow(TSharedPtr<FOpenVDBGridInfo> Item, const TSharedRef<STableViewBase>& OwnerTable);
	bool CanImport() const;
	FReply OnResetToDefaultClick();
	FText GetImportTypeDisplayText() const;
	void SetDefaultGridAssignment();
};