// Copyright Epic Games, Inc. All Rights Reserved.

#include "SparseVolumeTextureFactory.h"

#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "SparseVolumeTexture/SparseVolumeTextureData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SparseVolumeTextureFactory)

#if WITH_EDITOR

#include "SparseVolumeTextureOpenVDB.h"
#include "SparseVolumeTextureOpenVDBUtility.h"
#include "OpenVDBImportOptions.h"

#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/FileHelper.h"
#include "Async/Async.h"
#include "Async/ParallelFor.h"

#include "AssetImportTask.h"
#include "Editor.h"
#include "EditorFramework/AssetImportData.h"
#include "ObjectTools.h"

#include "OpenVDBImportWindow.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "Interfaces/IMainFrameModule.h"

#include <atomic>
#include <mutex>

#define LOCTEXT_NAMESPACE "USparseVolumeTextureFactory"

DEFINE_LOG_CATEGORY_STATIC(LogSparseVolumeTextureFactory, Log, All);

static void ComputeDefaultOpenVDBGridAssignment(const TArray<TSharedPtr<FOpenVDBGridComponentInfo>>& GridComponentInfo, int32 NumFiles, FOpenVDBImportOptions* ImportOptions)
{
	for (FOpenVDBSparseVolumeAttributesDesc& AttributesDesc : ImportOptions->Attributes)
	{
		for (FOpenVDBSparseVolumeComponentMapping& Mapping : AttributesDesc.Mappings)
		{
			Mapping.SourceGridIndex = INDEX_NONE;
			Mapping.SourceComponentIndex = INDEX_NONE;
		}
		AttributesDesc.Format = ESparseVolumeAttributesFormat::Float16;
	}

	// Assign the components of the input grids to the components of the output SVT.
	
	const TSharedPtr<FOpenVDBGridComponentInfo>* DensityComponentInfoPtr = GridComponentInfo.FindByPredicate([](const TSharedPtr<FOpenVDBGridComponentInfo>& GridComponent) { return GridComponent->Name == TEXT("density"); });
	const TSharedPtr<FOpenVDBGridComponentInfo>* NoneComponentInfoPtr = GridComponentInfo.FindByPredicate([](const TSharedPtr<FOpenVDBGridComponentInfo>& GridComponent) { return GridComponent->Name == TEXT("<None>"); });
	const int32 NumNonDensityComponentInfos = GridComponentInfo.Num() - (NoneComponentInfoPtr ? 1 : 0) - (DensityComponentInfoPtr ? 1 : 0);
	
	// Optimized density assignment: density as 8bit unorm in Attributes A and all other components in Attributes B as 16bit float. This only works if there is a maximum of 4 non-density components.
	// We also don't use this assignment if there are 3 non-density components as these will be padded to a 4 component format anyways, so we might as well put all 4 components into a single texture.
	const bool bOptimizedDensityAssignment = (NumNonDensityComponentInfos <= 4 && NumNonDensityComponentInfos != 3) && DensityComponentInfoPtr != nullptr;
	
	if (bOptimizedDensityAssignment)
	{
		// Assign density to the first channel of Attributes A and set format to 8 bit unorm
		ImportOptions->Attributes[0].Mappings[0].SourceGridIndex = (*DensityComponentInfoPtr)->Index;
		ImportOptions->Attributes[0].Mappings[0].SourceComponentIndex = (*DensityComponentInfoPtr)->ComponentIndex;
		ImportOptions->Attributes[0].Format = ESparseVolumeAttributesFormat::Unorm8;

		// All the other components go into Attributes B with 16 bit float
		uint32 DstComponentIdx = 0;
		for (const TSharedPtr<FOpenVDBGridComponentInfo>& GridComponent : GridComponentInfo)
		{
			if (GridComponent->Index == INDEX_NONE || &GridComponent == DensityComponentInfoPtr || !ensure(DstComponentIdx <= 3))
			{
				continue;
			}
			ImportOptions->Attributes[1].Mappings[DstComponentIdx].SourceGridIndex = GridComponent->Index;
			ImportOptions->Attributes[1].Mappings[DstComponentIdx].SourceComponentIndex = GridComponent->ComponentIndex;
			++DstComponentIdx;
		}
		ImportOptions->Attributes[1].Format = ESparseVolumeAttributesFormat::Float16;
	}
	else
	{
		uint32 DstAttributesIdx = 0;
		uint32 DstComponentIdx = 0;
		for (const TSharedPtr<FOpenVDBGridComponentInfo>& GridComponent : GridComponentInfo)
		{
			if (GridComponent->Index == INDEX_NONE)
			{
				continue;
			}
			ImportOptions->Attributes[DstAttributesIdx].Mappings[DstComponentIdx].SourceGridIndex = GridComponent->Index;
			ImportOptions->Attributes[DstAttributesIdx].Mappings[DstComponentIdx].SourceComponentIndex = GridComponent->ComponentIndex;
			++DstComponentIdx;
			if (DstComponentIdx == 4)
			{
				DstComponentIdx = 0;
				++DstAttributesIdx;
				if (DstAttributesIdx == 2)
				{
					break;
				}
			}
		}
	}
	
	ImportOptions->bIsSequence = NumFiles > 1;
}

bool LoadOpenVDBPreviewData(const FString& Filename, FOpenVDBPreviewData* OutPreviewData)
{
	FOpenVDBPreviewData& Result = *OutPreviewData;
	check(Result.LoadedFile.IsEmpty());
	check(Result.GridInfo.IsEmpty());
	check(Result.GridInfoPtrs.IsEmpty());
	check(Result.GridComponentInfoPtrs.IsEmpty());
	check(Result.SequenceFilenames.IsEmpty());

	if (!FFileHelper::LoadFileToArray(Result.LoadedFile, *Filename))
	{
		UE_LOGF(LogSparseVolumeTextureFactory, Error, "OpenVDB file could not be loaded: %ls", *Filename);
		return false;
	}
	if (!GetOpenVDBGridInfo(Result.LoadedFile, true /*bCreateStrings*/, &Result.GridInfo))
	{
		UE_LOGF(LogSparseVolumeTextureFactory, Error, "Failed to read OpenVDB file: %ls", *Filename);
		return false;
	}
	if (Result.GridInfo.IsEmpty())
	{
		UE_LOGF(LogSparseVolumeTextureFactory, Error, "OpenVDB file contains no grids: %ls", *Filename);
		return false;
	}

	// We need a <None> option to leave channels empty
	FOpenVDBGridComponentInfo NoneGridComponentInfo;
	NoneGridComponentInfo.Index = INDEX_NONE;
	NoneGridComponentInfo.ComponentIndex = INDEX_NONE;
	NoneGridComponentInfo.Name = TEXT("<None>");
	NoneGridComponentInfo.DisplayString = TEXT("<None>");

	Result.GridComponentInfoPtrs.Add(MakeShared<FOpenVDBGridComponentInfo>(NoneGridComponentInfo));

	// Create individual entries for each component of all valid source grids.
	// This is an array of TSharedPtr because SComboBox requires its input to be wrapped in TSharedPtr.
	bool bFoundSupportedGridType = false;
	for (const FOpenVDBGridInfo& Grid : Result.GridInfo)
	{
		// Append all grids, even if we don't actually support them
		Result.GridInfoPtrs.Add(MakeShared<FOpenVDBGridInfo>(Grid));

		if (Grid.Type == EOpenVDBGridType::Unknown || !IsOpenVDBGridValid(Grid, Filename))
		{
			continue;
		}

		bFoundSupportedGridType = true;

		// Create one entry per component
		for (uint32 ComponentIdx = 0; ComponentIdx < Grid.NumComponents; ++ComponentIdx)
		{
			FOpenVDBGridComponentInfo ComponentInfo;
			ComponentInfo.Index = Grid.Index;
			ComponentInfo.ComponentIndex = ComponentIdx;
			ComponentInfo.Name = Grid.Name;

			const TCHAR* ComponentNames[] = { TEXT(".X"), TEXT(".Y"),TEXT(".Z"),TEXT(".W") };
			FStringFormatOrderedArguments FormatArgs;
			FormatArgs.Add(ComponentInfo.Index);
			FormatArgs.Add(ComponentInfo.Name);
			FormatArgs.Add(Grid.NumComponents == 1 ? TEXT("") : ComponentNames[ComponentIdx]);

			ComponentInfo.DisplayString = FString::Format(TEXT("{0}. {1}{2}"), FormatArgs);

			Result.GridComponentInfoPtrs.Add(MakeShared<FOpenVDBGridComponentInfo>(MoveTemp(ComponentInfo)));
		}
	}

	if (!bFoundSupportedGridType)
	{
		UE_LOGF(LogSparseVolumeTextureFactory, Error, "OpenVDB file contains no grids of supported type: %ls", *Filename);
		return false;
	}

	Result.SequenceFilenames = FindOpenVDBSequenceFileNames(Filename);

	ComputeDefaultOpenVDBGridAssignment(Result.GridComponentInfoPtrs, Result.SequenceFilenames.Num(), &Result.DefaultImportOptions);

	return true;
}

static bool ShowOpenVDBImportWindow(const FString& Filename, const FOpenVDBPreviewData& PreviewData, FOpenVDBImportOptions* OutImportOptions)
{
	TSharedPtr<SWindow> ParentWindow;

	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}

	// Compute centered window position based on max window size, which include when all categories are expanded
	const float ImportWindowWidth = 450.0f;
	const float ImportWindowHeight = 750.0f;
	FVector2D ImportWindowSize = FVector2D(ImportWindowWidth, ImportWindowHeight); // Max window size it can get based on current slate


	FSlateRect WorkAreaRect = FSlateApplicationBase::Get().GetPreferredWorkArea();
	FVector2D DisplayTopLeft(WorkAreaRect.Left, WorkAreaRect.Top);
	FVector2D DisplaySize(WorkAreaRect.Right - WorkAreaRect.Left, WorkAreaRect.Bottom - WorkAreaRect.Top);

	float ScaleFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(DisplayTopLeft.X, DisplayTopLeft.Y);
	ImportWindowSize *= ScaleFactor;

	FVector2D WindowPosition = (DisplayTopLeft + (DisplaySize - ImportWindowSize) / 2.0f) / ScaleFactor;

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(NSLOCTEXT("UnrealEd", "OpenVDBImportOptionsTitle", "OpenVDB Import Options"))
		.SizingRule(ESizingRule::Autosized)
		.AutoCenter(EAutoCenter::None)
		.ClientSize(ImportWindowSize)
		.ScreenPosition(WindowPosition);

	TArray<TSharedPtr<ESparseVolumeAttributesFormat>> SupportedFormats =
	{
		MakeShared<ESparseVolumeAttributesFormat>(ESparseVolumeAttributesFormat::Float32),
		MakeShared<ESparseVolumeAttributesFormat>(ESparseVolumeAttributesFormat::Float16),
		MakeShared<ESparseVolumeAttributesFormat>(ESparseVolumeAttributesFormat::Unorm8)
	};

	TSharedPtr<SOpenVDBImportWindow> OpenVDBOptionWindow;
	Window->SetContent
	(
		SAssignNew(OpenVDBOptionWindow, SOpenVDBImportWindow)
		.ImportOptions(OutImportOptions)
		.DefaultImportOptions(&PreviewData.DefaultImportOptions)
		.NumFoundFiles(PreviewData.SequenceFilenames.Num())
		.OpenVDBGridInfo(&PreviewData.GridInfoPtrs)
		.OpenVDBGridComponentInfo(&PreviewData.GridComponentInfoPtrs)
		.OpenVDBSupportedTargetFormats(&SupportedFormats)
		.WidgetWindow(Window)
		.FullPath(FText::FromString(Filename))
		.MaxWindowHeight(ImportWindowHeight)
		.MaxWindowWidth(ImportWindowWidth)
	);

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

	OutImportOptions->bIsSequence = OpenVDBOptionWindow->ShouldImportAsSequence();

	return OpenVDBOptionWindow->ShouldImport();
}

static bool ValidateImportOptions(const FOpenVDBImportOptions& ImportOptions, const TArray<FOpenVDBGridInfo>& GridInfo)
{
	const int32 NumGrids = GridInfo.Num();

	for (const FOpenVDBSparseVolumeAttributesDesc& AttributesDesc : ImportOptions.Attributes)
	{
		for (const FOpenVDBSparseVolumeComponentMapping& Mapping : AttributesDesc.Mappings)
		{
			const int32 SourceGridIndex = Mapping.SourceGridIndex;
			const int32 SourceComponentIndex = Mapping.SourceComponentIndex;
			if (Mapping.SourceGridIndex != INDEX_NONE)
			{
				if (SourceGridIndex >= NumGrids)
				{
					return false; // Invalid grid index
				}
				if (SourceComponentIndex == INDEX_NONE || SourceComponentIndex >= (int32)GridInfo[SourceGridIndex].NumComponents)
				{
					return false; // Invalid component index
				}
			}
		}
	}
	return true;
}

USparseVolumeTextureFactory::USparseVolumeTextureFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	bEditorImport = true;
	SupportedClass = nullptr; // This factory supports multiple classes, so SupportedClass needs to be nullptr

	Formats.Add(TEXT("vdb;OpenVDB Format"));
}

FText USparseVolumeTextureFactory::GetDisplayName() const
{
	return LOCTEXT("SparseVolumeTextureFactoryDescription", "Sparse Volume Texture");
}

bool USparseVolumeTextureFactory::ConfigureProperties()
{
	return true;
}

bool USparseVolumeTextureFactory::ShouldShowInNewMenu() const
{
	return false;
}


///////////////////////////////////////////////////////////////////////////////
// Create asset


bool USparseVolumeTextureFactory::CanCreateNew() const
{
	return false;	// To be able to import files and call 
}

UObject* USparseVolumeTextureFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	USparseVolumeTexture* Object = NewObject<USparseVolumeTexture>(InParent, InClass, InName, Flags);

	// SVT_TODO initialize similarly to UTexture2DFactoryNew

	return Object;
}


///////////////////////////////////////////////////////////////////////////////
// Import asset


bool USparseVolumeTextureFactory::DoesSupportClass(UClass* Class)
{
	return Class == USparseVolumeTexture::StaticClass() || Class == UStaticSparseVolumeTexture::StaticClass() || Class == UAnimatedSparseVolumeTexture::StaticClass();
}

UClass* USparseVolumeTextureFactory::ResolveSupportedClass()
{
	// SVT_TODO: Do we need to return UStaticSparseVolumeTexture::StaticClass() or UAnimatedSparseVolumeTexture::StaticClass() here instead? Using the base class seems to work.
	return USparseVolumeTexture::StaticClass();
}

bool USparseVolumeTextureFactory::FactoryCanImport(const FString& Filename)
{
	const FString Extension = FPaths::GetExtension(Filename);
	if (Extension == TEXT("vdb"))
	{
		return true;
	}
	return false;
}

void USparseVolumeTextureFactory::CleanUp()
{
	Super::CleanUp();
}

UObject* USparseVolumeTextureFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename,
	const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	return ImportInternal(InClass, InParent, InName, Flags, Filename, Parms, bOutOperationCanceled, false /*bIsReimport*/);
}

bool USparseVolumeTextureFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
#if OPENVDB_AVAILABLE
	UStreamableSparseVolumeTexture* StreamableSVT = Cast<UStreamableSparseVolumeTexture>(Obj);
	if (StreamableSVT && StreamableSVT->AssetImportData)
	{
		StreamableSVT->AssetImportData->ExtractFilenames(OutFilenames);
		return true;
	}
#endif // OPENVDB_AVAILABLE
	return false;
}

void USparseVolumeTextureFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
#if OPENVDB_AVAILABLE
	UStreamableSparseVolumeTexture* StreamableSVT = Cast<UStreamableSparseVolumeTexture>(Obj);
	if (StreamableSVT && ensure(NewReimportPaths.Num() == 1))
	{
		StreamableSVT->AssetImportData->UpdateFilenameOnly(NewReimportPaths[0]);
	}
#endif // OPENVDB_AVAILABLE
}

EReimportResult::Type USparseVolumeTextureFactory::Reimport(UObject* Obj)
{
#if OPENVDB_AVAILABLE
	UStreamableSparseVolumeTexture* StreamableSVT = Cast<UStreamableSparseVolumeTexture>(Obj);
	if (!StreamableSVT)
	{
		return EReimportResult::Failed;
	}

	// Make sure file is valid and exists
	const FString Filename = StreamableSVT->AssetImportData->GetFirstFilename();
	if (!Filename.Len() || IFileManager::Get().FileSize(*Filename) == INDEX_NONE || !FactoryCanImport(Filename))
	{
		UE_LOGF(LogSparseVolumeTextureFactory, Error, "Reimport failed! Filename '%ls' is invalid or no such file exists.", *Filename);
		return EReimportResult::Failed;
	}

	bool OutCanceled = false;
	if (!ImportInternal(StreamableSVT->GetClass(), StreamableSVT->GetOuter(), *StreamableSVT->GetName(), RF_Public | RF_Standalone, Filename, nullptr, OutCanceled, true /*bIsReimport*/))
	{
		if (OutCanceled)
		{
			return EReimportResult::Cancelled;
		}

		return EReimportResult::Failed;
	}

	return EReimportResult::Succeeded;
#else
	UE_LOGF(LogSparseVolumeTextureFactory, Error, "Cannot import OpenVDB asset any platform other than Windows.");
	return EReimportResult::Failed;
#endif // OPENVDB_AVAILABLE
}

UObject* USparseVolumeTextureFactory::ImportWithTransform(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, bool& bOutOperationCanceled, const FTransform& AdditionalFrameTransform)
{
	return ImportInternal(InClass, InParent, InName, Flags, Filename, Parms, bOutOperationCanceled, false /*bIsReimport*/, AdditionalFrameTransform);
}

UObject* USparseVolumeTextureFactory::ImportInternal(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, bool& bOutOperationCanceled, bool bIsReimport, const FTransform& AdditionalFrameTransform)
{
#if OPENVDB_AVAILABLE

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, Parms);
	TArray<UObject*> ResultAssets;

	bOutOperationCanceled = false;

	const bool bIsUnattended = (IsAutomatedImport()
		|| FApp::IsUnattended()
		|| IsRunningCommandlet()
		|| GIsRunningUnattendedScript);

	FOpenVDBPreviewData PreviewData;

	// Use the provided preview data, if any
	bool bCollectedData = false;
	if (bIsUnattended && AssetImportTask)
	{
		if (UOpenVDBImportOptionsObject* TaskOptions = Cast<UOpenVDBImportOptionsObject>(AssetImportTask->Options))
		{
			PreviewData = TaskOptions->PreviewData;
			bCollectedData = true;
		}
	}

	// Otherwise, load file and get info about each contained grid
	if (!bCollectedData)
	{
		if (!LoadOpenVDBPreviewData(Filename, &PreviewData))
		{
			return nullptr;
		}
	}

	FOpenVDBImportOptions ImportOptions = PreviewData.DefaultImportOptions;

	if (!bIsUnattended)
	{
		// Show dialog for import options
		if (!ShowOpenVDBImportWindow(Filename, PreviewData, &ImportOptions))
		{
			bOutOperationCanceled = true;
			return nullptr;
		}
	}

	if (!ValidateImportOptions(ImportOptions, PreviewData.GridInfo))
	{
		UE_LOGF(LogSparseVolumeTextureFactory, Error, "Import options are invalid! This is likely due to invalid/out-of-bounds grid or component indices.");
		return nullptr;
	}

	// Utility function for computing the bounding box encompassing the bounds of all frames in the SVT.
	auto ExpandVolumeBounds = [](const FOpenVDBImportOptions& ImportOptions, const TArray<FOpenVDBGridInfo>& GridInfoArray, FIntVector3& VolumeBoundsMin, FIntVector3& VolumeBoundsMax)
	{
		for (const FOpenVDBSparseVolumeAttributesDesc& Attributes : ImportOptions.Attributes)
		{
			for (const FOpenVDBSparseVolumeComponentMapping& Mapping : Attributes.Mappings)
			{
				if (Mapping.SourceGridIndex != INDEX_NONE)
				{
					const FOpenVDBGridInfo& GridInfo = GridInfoArray[Mapping.SourceGridIndex];
					VolumeBoundsMin.X = FMath::Min(VolumeBoundsMin.X, GridInfo.VolumeActiveAABBMin.X);
					VolumeBoundsMin.Y = FMath::Min(VolumeBoundsMin.Y, GridInfo.VolumeActiveAABBMin.Y);
					VolumeBoundsMin.Z = FMath::Min(VolumeBoundsMin.Z, GridInfo.VolumeActiveAABBMin.Z);

					VolumeBoundsMax.X = FMath::Max(VolumeBoundsMax.X, GridInfo.VolumeActiveAABBMax.X);
					VolumeBoundsMax.Y = FMath::Max(VolumeBoundsMax.Y, GridInfo.VolumeActiveAABBMax.Y);
					VolumeBoundsMax.Z = FMath::Max(VolumeBoundsMax.Z, GridInfo.VolumeActiveAABBMax.Z);
				}
			}
		}
	};

	FIntVector3 VolumeBoundsMin = FIntVector3(INT32_MAX, INT32_MAX, INT32_MAX);
	FIntVector3 VolumeBoundsMax = FIntVector3(INT32_MIN, INT32_MIN, INT32_MIN);

	// Import as either single static SVT or a sequence of frames, making up an animated SVT
	if (!ImportOptions.bIsSequence)
	{
		// Import as a static sparse volume texture asset.

		FScopedSlowTask ImportTask(1.0f, LOCTEXT("ImportingVDBStatic", "Importing static OpenVDB"));
		ImportTask.MakeDialog(true);

		ExpandVolumeBounds(ImportOptions, PreviewData.GridInfo, VolumeBoundsMin, VolumeBoundsMax);

		UE::SVT::FTextureData TextureData{};
		FTransform FrameTransform = FTransform::Identity;
		const bool bConversionSuccess = ConvertOpenVDBToSparseVolumeTexture(PreviewData.LoadedFile, ImportOptions, VolumeBoundsMin, TextureData, FrameTransform);

		if (!bConversionSuccess)
		{
			UE_LOGF(LogSparseVolumeTextureFactory, Error, "Failed to convert OpenVDB file to SparseVolumeTexture: %ls", *Filename);
			return nullptr;
		}

		FrameTransform = FrameTransform * AdditionalFrameTransform;

		UStaticSparseVolumeTexture* StaticSVTexture = NewObject<UStaticSparseVolumeTexture>(InParent, UStaticSparseVolumeTexture::StaticClass(), InName, Flags);
		const bool bInitSuccess = StaticSVTexture->Initialize(MakeArrayView(&TextureData, 1), MakeArrayView(&FrameTransform, 1));
		if (!bInitSuccess)
		{
			UE_LOGF(LogSparseVolumeTextureFactory, Error, "Failed to initialize SparseVolumeTexture: %ls", *Filename);
			return nullptr;
		}

		if (ImportTask.ShouldCancel())
		{
			bOutOperationCanceled = true;
			return nullptr;
		}
		ImportTask.EnterProgressFrame(1.0f, LOCTEXT("ConvertingVDBStatic", "Converting static OpenVDB"));

		ResultAssets.Add(StaticSVTexture);
	}
	else
	{
		// Import as an animated sparse volume texture asset.

		// Data from original file is no longer needed; we iterate over all frames later
		PreviewData.LoadedFile.Empty();

		const int32 NumFrames = PreviewData.SequenceFilenames.Num();

		FScopedSlowTask ImportTask(NumFrames + 1, LOCTEXT("ImportingVDBAnim", "Importing OpenVDB animation"));
		ImportTask.MakeDialog(true);

		// Allocate space for each frame
		TArray<UE::SVT::FTextureData> UncookedFramesData;
		TArray<FTransform> FrameTransforms;
		UncookedFramesData.SetNum(NumFrames);
		FrameTransforms.SetNum(NumFrames);

		std::atomic_bool bErrored = false;
		std::atomic_bool bCanceled = false;

		// Compute volume bounds and check sequence files for compatiblity
		std::mutex VolumeBoundsMutex;
		ParallelFor(NumFrames, [&bErrored, &VolumeBoundsMutex, &VolumeBoundsMin, &VolumeBoundsMax, &ExpandVolumeBounds, &PreviewData, &ImportOptions](int32 FrameIdx)
			{
				if (bErrored.load())
				{
					return;
				}

				// Load file and get info about each contained grid
				const FString& FrameFilename = PreviewData.SequenceFilenames[FrameIdx];
				TArray64<uint8> LoadedFrameFile;
				if (!FFileHelper::LoadFileToArray(LoadedFrameFile, *FrameFilename))
				{
					UE_LOGF(LogSparseVolumeTextureFactory, Error, "OpenVDB file could not be loaded: %ls", *FrameFilename);
					bErrored.store(true);
					return;
				}

				TArray<FOpenVDBGridInfo> FrameGridInfo;
				if (!GetOpenVDBGridInfo(LoadedFrameFile, true, &FrameGridInfo))
				{
					UE_LOGF(LogSparseVolumeTextureFactory, Error, "Failed to read OpenVDB file: %ls", *FrameFilename);
					bErrored.store(true);
					return;
				}

				// Sanity check for compatibility
				for (const FOpenVDBSparseVolumeAttributesDesc& AttributesDesc : ImportOptions.Attributes)
				{
					for (const FOpenVDBSparseVolumeComponentMapping& Mapping : AttributesDesc.Mappings)
					{
						const uint32 SourceGridIndex = Mapping.SourceGridIndex;
						if (SourceGridIndex != INDEX_NONE)
						{
							if ((int32)SourceGridIndex >= FrameGridInfo.Num())
							{
								UE_LOGF(LogSparseVolumeTextureFactory, Error, "OpenVDB file is incompatible with other frames in the sequence: %ls", *FrameFilename);
								bErrored.store(true);
								return;
							}
							const FOpenVDBGridInfo& OrigSourceGrid = PreviewData.GridInfo[SourceGridIndex];
							const FOpenVDBGridInfo& FrameSourceGrid = FrameGridInfo[SourceGridIndex];
							if (OrigSourceGrid.Type != FrameSourceGrid.Type || OrigSourceGrid.Name != FrameSourceGrid.Name)
							{
								UE_LOGF(LogSparseVolumeTextureFactory, Error, "OpenVDB file is incompatible with other frames in the sequence: %ls", *FrameFilename);
								bErrored.store(true);
								return;
							}
						}
					}
				}

				// Update sequence volume bounds and increment ProcessedFramesCounter
				{
					std::lock_guard<std::mutex> Lock(VolumeBoundsMutex);
					ExpandVolumeBounds(ImportOptions, FrameGridInfo, VolumeBoundsMin, VolumeBoundsMax);
				}
			});

		if (bErrored.load())
		{
			return nullptr;
		}

		ImportTask.EnterProgressFrame(1.0f, LOCTEXT("ConvertingVDBAnim", "Converting OpenVDB animation"));

		FEvent* AllTasksFinishedEvent = FPlatformProcess::GetSynchEventFromPool();
		std::atomic_int FinishedTasksCounter = 0; // Will be incremented even if frame processing failed
		std::atomic_int ProcessedFramesCounter = 0;

		// Load individual frames, process/convert them and append them to the resulting asset
		for (int32 FrameIdx = 0; FrameIdx < NumFrames; ++FrameIdx)
		{
			// Increments the atomic counter when going out of scope. Triggers an event once the counter reaches a given value.
			struct FScopedIncrementer
			{
				std::atomic_int& Counter;
				int32 MaxValue;
				FEvent* Event;
				explicit FScopedIncrementer(std::atomic_int& InCounter, int32 InMaxValue, FEvent* InEvent)
					: Counter(InCounter), MaxValue(InMaxValue), Event(InEvent) {}
				~FScopedIncrementer()
				{
					if ((Counter.fetch_add(1) + 1) == MaxValue)
					{
						Event->Trigger();
					}
				}
			};

			AsyncTask(ENamedThreads::AnyNormalThreadNormalTask,
				[FrameIdx, NumFrames, &PreviewData, &ImportOptions, &UncookedFramesData, &FrameTransforms,
				AllTasksFinishedEvent, &bErrored, &bCanceled, &FinishedTasksCounter, &ProcessedFramesCounter, &VolumeBoundsMin, &AdditionalFrameTransform]()
				{
					// Ensure the FinishedTasksCounter will be incremented in all cases
					FScopedIncrementer Incremeter(FinishedTasksCounter, NumFrames, AllTasksFinishedEvent);

					if (bErrored.load() || bCanceled.load())
					{
						return;
					}

					const FString& FrameFilename = PreviewData.SequenceFilenames[FrameIdx];

					UE_LOGF(LogSparseVolumeTextureFactory, Display, "Loading OpenVDB sequence frame #%i %ls.", FrameIdx, *FrameFilename);

					// Load file
					TArray64<uint8> LoadedFrameFile;
					if (!FFileHelper::LoadFileToArray(LoadedFrameFile, *FrameFilename))
					{
						UE_LOGF(LogSparseVolumeTextureFactory, Error, "OpenVDB file could not be loaded: %ls", *FrameFilename);
						bErrored.store(true);
						return;
					}

					UE::SVT::FTextureData TextureData{};
					FTransform FrameTransform = FTransform::Identity;
					const bool bConversionSuccess = ConvertOpenVDBToSparseVolumeTexture(LoadedFrameFile, ImportOptions, VolumeBoundsMin, TextureData, FrameTransform);

					if (!bConversionSuccess)
					{
						UE_LOGF(LogSparseVolumeTextureFactory, Error, "Failed to convert OpenVDB file to SparseVolumeTexture: %ls", *FrameFilename);
						bErrored.store(true);
						return;
					}

					UncookedFramesData[FrameIdx] = MoveTemp(TextureData);
					FrameTransforms[FrameIdx] = FrameTransform * AdditionalFrameTransform;

					// Increment ProcessedFramesCounter
					ProcessedFramesCounter.fetch_add(1);
				});
		}

		// Wait for frames to be processed
		{
			int NumFinishedTasks = 0;
			int NumProcessedFrames = 0;

			while (NumFinishedTasks < NumFrames)
			{
				// We can't block here because we want to regularly update the progress bar and check for user input.
				const uint32 WaitTimeMS = 2;
				AllTasksFinishedEvent->Wait(WaitTimeMS);

				if (!bCanceled.load() && !bErrored.load() && ImportTask.ShouldCancel())
				{
					bCanceled.store(true);
				}

				const int NewNumFinishedTasks = FinishedTasksCounter.load();
				if (NewNumFinishedTasks > NumFinishedTasks)
				{
					const int NewNumProcessedFrames = ProcessedFramesCounter.load();
					if (NewNumProcessedFrames > NumProcessedFrames && !bErrored.load())
					{
						const float Progress = float(NewNumProcessedFrames - NumProcessedFrames);
						ImportTask.EnterProgressFrame(Progress, LOCTEXT("ConvertingVDBAnim", "Converting OpenVDB animation"));
					}

					NumFinishedTasks = NewNumFinishedTasks;
					NumProcessedFrames = NewNumProcessedFrames;
				}
			}
		}
		FPlatformProcess::ReturnSynchEventToPool(AllTasksFinishedEvent);

		if (bCanceled.load())
		{
			bOutOperationCanceled = true;
			return nullptr;
		}
		if (bErrored.load())
		{
			return nullptr;
		}

		// By default, the resulting package (already created) and object (about to be) will have the name of the imported file.
		// However, since the file is part of an entire sequence that we imported, it would be confusing if the resulting asset was named "file_0000" instead of "file",
		// so we attempt to rename both package and object here.
		// Don't try to rename the SVT if we are doing a reimport.
		FName NewObjectName = InName;
		if (!bIsReimport)
		{
			FString NewFileName = GetVDBSequenceBaseFileName(Filename, false /*bDiscardNumbersOnly*/);
			// GetVDBSequenceBaseFileName() discards the number as well as underscores and invalid chars at the end of the filename.
			// We still need to ensure that there are no additional invalid characters in the filename.
			NewFileName = ObjectTools::SanitizeObjectName(NewFileName);

			// Don't try to rename the package if it's the transient package or the new filename is empty
			if (!NewFileName.IsEmpty() && (InParent != GetTransientPackage() && InParent->IsA<UPackage>()))
			{
				const FString PackageName = InParent->GetName(); // Contains name with path
				int32 LastSeparatorIndex = 0;
				const bool bFoundSeparator = PackageName.FindLastChar(TEXT('/'), LastSeparatorIndex);
				// Get the substring containing the path only, without the file name
				const FString PackagePath = bFoundSeparator ? PackageName.Left(LastSeparatorIndex) : FString();
				const FString NewPackageName = PackagePath / NewFileName;

				UPackage* ExistingPackage = FindPackage(InParent->GetOuter(), *NewPackageName);
				if (!ExistingPackage)
				{
					InParent->Rename(*NewPackageName, nullptr, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
					NewObjectName = *NewFileName;
				}
			}
		}

		UAnimatedSparseVolumeTexture* AnimatedSVTexture = NewObject<UAnimatedSparseVolumeTexture>(InParent, UAnimatedSparseVolumeTexture::StaticClass(), NewObjectName, Flags);
		const bool bInitSuccess = AnimatedSVTexture->Initialize(UncookedFramesData, FrameTransforms);
		if (!bInitSuccess)
		{
			UE_LOGF(LogSparseVolumeTextureFactory, Error, "Failed to initialize SparseVolumeTexture: %ls", *Filename);
			return nullptr;
		}

		ResultAssets.Add(AnimatedSVTexture);
	}

	// Now notify the system about the imported/updated/created assets
	check(ResultAssets.Num() == 1);
	CastChecked<UStreamableSparseVolumeTexture>(ResultAssets[0])->AssetImportData->Update(Filename);
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, ResultAssets[0]);

	return ResultAssets[0];

#else // OPENVDB_AVAILABLE

	// SVT_TODO Make sure we can also import on more platforms such as Linux. See SparseVolumeTextureOpenVDB.h
	UE_LOGF(LogSparseVolumeTextureFactory, Error, "Cannot import OpenVDB asset any platform other than Windows.");
	return nullptr;

#endif // OPENVDB_AVAILABLE
}

#endif // WITH_EDITORONLY_DATA

#undef LOCTEXT_NAMESPACE
