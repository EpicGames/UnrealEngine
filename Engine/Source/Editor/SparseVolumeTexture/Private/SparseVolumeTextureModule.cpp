// Copyright Epic Games, Inc. All Rights Reserved.

#include "SparseVolumeTextureModule.h"

#include "SparseVolumeTextureOpenVDB.h"

#define LOCTEXT_NAMESPACE "SparseVolumeTextureModule"

IMPLEMENT_MODULE(FSparseVolumeTextureModule, SparseVolumeTexture);

template<typename T>
static void RegisterOpenVDBGrid()
{
	if (!T::isRegistered())
	{
		T::registerGrid();
	}
}

void FSparseVolumeTextureModule::StartupModule()
{
#if OPENVDB_AVAILABLE
	// Global registration of  the vdb types.
	openvdb::initialize();

	RegisterOpenVDBGrid<FOpenVDBHalf1Grid>();
	RegisterOpenVDBGrid<FOpenVDBHalf2Grid>();
	RegisterOpenVDBGrid<FOpenVDBHalf3Grid>();
	RegisterOpenVDBGrid<FOpenVDBHalf4Grid>();
	RegisterOpenVDBGrid<FOpenVDBFloat1Grid>();
	RegisterOpenVDBGrid<FOpenVDBFloat2Grid>();
	RegisterOpenVDBGrid<FOpenVDBFloat3Grid>();
	RegisterOpenVDBGrid<FOpenVDBFloat4Grid>();
	RegisterOpenVDBGrid<FOpenVDBDouble1Grid>();
	RegisterOpenVDBGrid<FOpenVDBDouble2Grid>();
	RegisterOpenVDBGrid<FOpenVDBDouble3Grid>();
	RegisterOpenVDBGrid<FOpenVDBDouble4Grid>();
#endif
}

void FSparseVolumeTextureModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
