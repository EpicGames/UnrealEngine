// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/CoreMiscDefines.h" // UE_PUSH_MACRO seems to break without this

#if OPENVDB_AVAILABLE

THIRD_PARTY_INCLUDES_START

UE_PUSH_MACRO("check")
#undef check

#ifdef _MSC_VER
#pragma warning (push)
#pragma warning (disable : 4146) // unary minus operator applied to unsigned type, result still unsigned
#pragma warning (disable : 6297) // Arithmetic overflow:  32-bit value is shifted, then cast to 64-bit value.  Results might not be an expected value.
#endif

#ifndef M_PI
#define M_PI    3.14159265358979323846
#define LOCAL_M_PI 1
#endif

#ifndef M_PI_2
#define M_PI_2  1.57079632679489661923 // pi/2
#define LOCAL_M_PI_2 1
#endif

#define BOOST_ALLOW_DEPRECATED_HEADERS

#include <openvdb/openvdb.h>
#include <openvdb/io/Stream.h>
#include <openvdb/io/File.h>
#include <openvdb/tools/Prune.h>
#include <openvdb/tools/Dense.h>
#include <openvdb/math/Vec4.h>
#include <openvdb/math/Half.h>
#include <openvdb/Exceptions.h>

using FOpenVDBHalf = openvdb::math::internal::half;
using FOpenVDBHalf1Grid = openvdb::Grid<openvdb::tree::Tree4<FOpenVDBHalf, 5, 4, 3>::Type>;
using FOpenVDBHalf2Grid = openvdb::Grid<openvdb::tree::Tree4<openvdb::math::Vec2<FOpenVDBHalf>, 5, 4, 3>::Type>;
using FOpenVDBHalf3Grid = openvdb::Grid<openvdb::tree::Tree4<openvdb::math::Vec3<FOpenVDBHalf>, 5, 4, 3>::Type>;
using FOpenVDBHalf4Grid = openvdb::Grid<openvdb::tree::Tree4<openvdb::math::Vec4<FOpenVDBHalf>, 5, 4, 3>::Type>;
using FOpenVDBFloat1Grid = openvdb::FloatGrid;
using FOpenVDBFloat2Grid = openvdb::Grid<openvdb::tree::Tree4<openvdb::Vec2f, 5, 4, 3>::Type>;
using FOpenVDBFloat3Grid = openvdb::Vec3SGrid;
using FOpenVDBFloat4Grid = openvdb::Grid<openvdb::tree::Tree4<openvdb::Vec4f, 5, 4, 3>::Type>;
using FOpenVDBDouble1Grid = openvdb::DoubleGrid;
using FOpenVDBDouble2Grid = openvdb::Grid<openvdb::tree::Tree4<openvdb::Vec2d, 5, 4, 3>::Type>;
using FOpenVDBDouble3Grid = openvdb::Vec3DGrid;
using FOpenVDBDouble4Grid = openvdb::Grid<openvdb::tree::Tree4<openvdb::Vec4d, 5, 4, 3>::Type>;

#if LOCAL_M_PI
#undef M_PI
#endif

#if LOCAL_M_PI_2
#undef M_PI_2
#endif

#ifdef _MSC_VER
#pragma warning (pop)
#endif

UE_POP_MACRO("check")

THIRD_PARTY_INCLUDES_END

#endif
