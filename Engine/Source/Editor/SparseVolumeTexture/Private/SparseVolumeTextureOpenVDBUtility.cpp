// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "SparseVolumeTextureOpenVDBUtility.h"
#include "SparseVolumeTextureOpenVDB.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "SparseVolumeTexture/SparseVolumeTextureData.h"
#include "OpenVDBGridAdapter.h"
#include "OpenVDBImportOptions.h"

#include "HAL/FileManager.h"
#include "Misc/Paths.h"

DEFINE_LOG_CATEGORY_STATIC(LogSparseVolumeTextureOpenVDBUtility, Log, All);

#if OPENVDB_AVAILABLE

namespace
{
	// Utility class acting as adapter between TArray64<uint8> and std::istream.
	// In order to work around a problem where a std::streambuf implementation is limited to
	// 2GB buffers, we need to manually update the pointers whenever we are done processing a 2GB chunk.
	class FArrayUint8StreamBuf : public std::streambuf
	{
	public:
		explicit FArrayUint8StreamBuf(TArray64<uint8>& Array)
		{
			char* Data = (char*)Array.GetData();
			size_t Num = Array.Num();
			FileBegin = Data;
			FileEnd = Data + Num;
			FileRead = Data;
		}

		// Calls setg() to with a set of pointers exposing a window into the input file.
		// Returns true if there are any more bytes to be read.
		bool UpdatePointers()
		{
			StreamBegin = FileRead;
			StreamEnd = FMath::Min(FileEnd, StreamBegin + ChunkSize);
			StreamRead = StreamBegin;
			FileRead = StreamRead;
			setg(StreamBegin, StreamRead, StreamEnd);
			const bool bHasBytesToRead = StreamBegin < StreamEnd;
			return bHasBytesToRead;
		}

		// This function is called by the parent class and is expected to be implemented by subclasses.
		// It requests n bytes to be copied into s. 
		std::streamsize xsgetn(char* s, std::streamsize n) override
		{
			for (std::streamsize ReadBytes = 0; ReadBytes < n;)
			{
				// We read all bytes of the input file. gptr() is the current read ptr and egptr() is the end ptr.
				if (gptr() == egptr() && !UpdatePointers())
				{
					check(gptr() == FileEnd);
					return ReadBytes;
				}
				// Try to read n bytes but make a smaller read if we would read past the current ptr window exposed to streambuf.
				const std::streamsize Available = FMath::Min(n - ReadBytes, static_cast<std::streamsize>(egptr() - gptr()));
				memcpy(s, gptr(), Available);
				// Advance pointers and the ReadBytes counter
				s += Available;
				ReadBytes += Available;
				StreamRead = gptr() + Available;
				FileRead = StreamRead;
				// Update the Next ptr in streambuf
				setg(StreamBegin, StreamRead, StreamEnd);
			}
			return n;
		}

		// Get the current character but don't advance the position. This is called by uflow() in the parent class when it runs out of bytes.
		// We use it to move the exposed window into the file data.
		int_type underflow() override
		{
			return (gptr() == egptr() && !UpdatePointers()) ? traits_type::eof() : *gptr();
		}

	private:
		static constexpr size_t ChunkSize = INT32_MAX;	// The size of the range to expose to std::streambuf with setg()
		char* FileBegin = nullptr;						// Begin ptr of the file data
		char* FileEnd = nullptr;						// One byte past the end of the file data
		char* FileRead = nullptr;						// The position up to which the file data has been exposed to/processed by the streambuf.
		char* StreamBegin = nullptr;					// The begin ptr of the currently exposed file chunk.
		char* StreamEnd = nullptr;						// The end ptr of the currently exposed file chunk.
		char* StreamRead = nullptr;						// The last value of the read/next ptr set with setg().
	};
}

static FOpenVDBGridInfo GetOpenVDBGridInfo(openvdb::GridBase::Ptr Grid, uint32 GridIndex, bool bCreateStrings)
{
	openvdb::CoordBBox VolumeActiveAABB = Grid->evalActiveVoxelBoundingBox();
	openvdb::Coord VolumeActiveDim = Grid->evalActiveVoxelDim();
	openvdb::math::MapBase::ConstPtr MapBase = Grid->constTransform().baseMap();
	openvdb::Vec3d VoxelSize = MapBase->voxelSize();
	openvdb::Mat4d GridTransformVDB = MapBase->getAffineMap()->getConstMat4();

	FOpenVDBGridInfo GridInfo;
	GridInfo.Index = GridIndex;
	GridInfo.NumComponents = 0;
	GridInfo.Type = EOpenVDBGridType::Unknown;
	GridInfo.VolumeActiveAABBMin = FIntVector3(VolumeActiveAABB.min().x(), VolumeActiveAABB.min().y(), VolumeActiveAABB.min().z());
	GridInfo.VolumeActiveAABBMax = FIntVector3(VolumeActiveAABB.max().x() + 1, VolumeActiveAABB.max().y() + 1, VolumeActiveAABB.max().z() + 1); // +1 because CoordBBox::Max is inclusive, but we want exclusive
	GridInfo.VolumeActiveDim = FIntVector3(VolumeActiveDim.x(), VolumeActiveDim.y(), VolumeActiveDim.z());
	GridInfo.bIsInWorldSpace = Grid->isInWorldSpace();

	FMatrix TransformMatrix;
	for (int i = 0; i < 4; ++i)
	{
		for (int j = 0; j < 4; ++j)
		{
			TransformMatrix.M[i][j] = GridTransformVDB[i][j];
		}
	}
	GridInfo.Transform = FTransform(TransformMatrix);

	// Figure out the type/format of the grid
	if (Grid->isType<FOpenVDBHalf1Grid>())
	{
		GridInfo.NumComponents = 1;
		GridInfo.Type = EOpenVDBGridType::Half;
	}
	else if (Grid->isType<FOpenVDBHalf2Grid>())
	{
		GridInfo.NumComponents = 2;
		GridInfo.Type = EOpenVDBGridType::Half2;
	}
	else if (Grid->isType<FOpenVDBHalf3Grid>())
	{
		GridInfo.NumComponents = 3;
		GridInfo.Type = EOpenVDBGridType::Half3;
	}
	else if (Grid->isType<FOpenVDBHalf4Grid>())
	{
		GridInfo.NumComponents = 4;
		GridInfo.Type = EOpenVDBGridType::Half4;
	}
	else if (Grid->isType<FOpenVDBFloat1Grid>())
	{
		GridInfo.NumComponents = 1;
		GridInfo.Type = EOpenVDBGridType::Float;
	}
	else if (Grid->isType<FOpenVDBFloat2Grid>())
	{
		GridInfo.NumComponents = 2;
		GridInfo.Type = EOpenVDBGridType::Float2;
	}
	else if (Grid->isType<FOpenVDBFloat3Grid>())
	{
		GridInfo.NumComponents = 3;
		GridInfo.Type = EOpenVDBGridType::Float3;
	}
	else if (Grid->isType<FOpenVDBFloat4Grid>())
	{
		GridInfo.NumComponents = 4;
		GridInfo.Type = EOpenVDBGridType::Float4;
	}
	else if (Grid->isType<FOpenVDBDouble1Grid>())
	{
		GridInfo.NumComponents = 1;
		GridInfo.Type = EOpenVDBGridType::Double;
	}
	else if (Grid->isType<FOpenVDBDouble2Grid>())
	{
		GridInfo.NumComponents = 2;
		GridInfo.Type = EOpenVDBGridType::Double2;
	}
	else if (Grid->isType<FOpenVDBDouble3Grid>())
	{
		GridInfo.NumComponents = 3;
		GridInfo.Type = EOpenVDBGridType::Double3;
	}
	else if (Grid->isType<FOpenVDBDouble4Grid>())
	{
		GridInfo.NumComponents = 4;
		GridInfo.Type = EOpenVDBGridType::Double4;
	}

	if (bCreateStrings)
	{
		GridInfo.Name = Grid->getName().c_str();

		FStringFormatOrderedArguments FormatArgs;
		FormatArgs.Add(GridInfo.Index);
		FormatArgs.Add(OpenVDBGridTypeToString(GridInfo.Type));
		FormatArgs.Add(GridInfo.Name);

		GridInfo.DisplayString = FString::Format(TEXT("{0}. Type: {1}, Name: \"{2}\""), FormatArgs);
	}

	return GridInfo;
}

#endif

bool IsOpenVDBGridValid(const FOpenVDBGridInfo& GridInfo, const FString& Filename)
{
	return true;
}

bool GetOpenVDBGridInfo(TArray64<uint8>& SourceFile, bool bCreateStrings, TArray<FOpenVDBGridInfo>* OutGridInfo)
{
#if OPENVDB_AVAILABLE
	FArrayUint8StreamBuf StreamBuf(SourceFile);
	std::istream IStream(&StreamBuf);
	openvdb::io::Stream Stream;
	try
	{
		Stream = openvdb::io::Stream(IStream, false /*delayLoad*/);
	}
	catch (const openvdb::Exception& Exception)
	{
		UE_LOGF(LogSparseVolumeTextureOpenVDBUtility, Error, "Failed to read file due to exception: %ls", *FString(Exception.what()));
		return false;
	}

	openvdb::GridPtrVecPtr Grids = Stream.getGrids();

	OutGridInfo->Empty(Grids->size());

	uint32 GridIndex = 0;
	for (openvdb::GridBase::Ptr& Grid : *Grids)
	{
		OutGridInfo->Add(GetOpenVDBGridInfo(Grid, GridIndex, bCreateStrings));
		++GridIndex;
	}

	return true;
#else
	return false;
#endif // OPENVDB_AVAILABLE
}

static EPixelFormat GetMultiComponentFormat(ESparseVolumeAttributesFormat Format, uint32 NumComponents)
{
	switch (Format)
	{
	case ESparseVolumeAttributesFormat::Unorm8:
	{
		switch (NumComponents)
		{
		case 1: return PF_R8;
		case 2: return PF_R8G8;
		case 3:
		case 4: return PF_R8G8B8A8;
		}
		break;
	}
	case ESparseVolumeAttributesFormat::Float16:
	{
		switch (NumComponents)
		{
		case 1: return PF_R16F;
		case 2: return PF_G16R16F;
		case 3:
		case 4: return PF_FloatRGBA;
		}
		break;
	}
	case ESparseVolumeAttributesFormat::Float32:
	{
		switch (NumComponents)
		{
		case 1: return PF_R32_FLOAT;
		case 2: return PF_G32R32F;
		case 3:
		case 4: return PF_A32B32G32R32F;
		}
		break;
	}
	}
	return PF_Unknown;
}


#if OPENVDB_AVAILABLE

class FSparseVolumeTextureDataProviderOpenVDB : public UE::SVT::ITextureDataProvider
{
public:

	bool Initialize(TArray64<uint8>& SourceFile, const FOpenVDBImportOptions& ImportOptions, const FIntVector3& InVolumeBoundsMin)
	{
		Attributes = ImportOptions.Attributes;
		VolumeBoundsMin = InVolumeBoundsMin;

		// Compute some basic info about the number of components and which format to use
		EPixelFormat MultiCompFormat[NumAttributesDescs] = {};
		bool bNormalizedFormat[NumAttributesDescs] = {};
		bool bHasValidSourceGrids[NumAttributesDescs] = {};
		bool bAnySourceGridIndicesValid = false;

		for (int32 AttributesIdx = 0; AttributesIdx < NumAttributesDescs; ++AttributesIdx)
		{
			int32 NumRequiredComponents = 0;
			for (int32 ComponentIdx = 0; ComponentIdx < 4; ++ComponentIdx)
			{
				if (Attributes[AttributesIdx].Mappings[ComponentIdx].SourceGridIndex != INDEX_NONE)
				{
					check(Attributes[AttributesIdx].Mappings[ComponentIdx].SourceComponentIndex != INDEX_NONE);
					NumRequiredComponents = FMath::Max(ComponentIdx + 1, NumRequiredComponents);
					bHasValidSourceGrids[AttributesIdx] = true;
					bAnySourceGridIndicesValid = true;
				}
			}

			if (bHasValidSourceGrids[AttributesIdx])
			{
				NumComponents[AttributesIdx] = NumRequiredComponents == 3 ? 4 : NumRequiredComponents; // We don't support formats with only 3 components
				bNormalizedFormat[AttributesIdx] = Attributes[AttributesIdx].Format == ESparseVolumeAttributesFormat::Unorm8;
				MultiCompFormat[AttributesIdx] = GetMultiComponentFormat(Attributes[AttributesIdx].Format, NumComponents[AttributesIdx]);

				if (MultiCompFormat[AttributesIdx] == PF_Unknown)
				{
					UE_LOGF(LogSparseVolumeTextureOpenVDBUtility, Warning, "SparseVolumeTexture is set to use an unsupported format: %i", (int32)Attributes[AttributesIdx].Format);
					return false;
				}
			}
		}

		// All source grid indices are INDEX_NONE, so nothing was selected for import
		if (!bAnySourceGridIndicesValid)
		{
			UE_LOGF(LogSparseVolumeTextureOpenVDBUtility, Warning, "SparseVolumeTexture has all components set to <None>, so there is nothing to import.");
			return false;
		}

		// Load file
		FArrayUint8StreamBuf StreamBuf(SourceFile);
		std::istream IStream(&StreamBuf);
		openvdb::io::Stream Stream;
		try
		{
			Stream = openvdb::io::Stream(IStream, false /*delayLoad*/);
		}
		catch (const openvdb::Exception& Exception)
		{
			UE_LOGF(LogSparseVolumeTextureOpenVDBUtility, Error, "Failed to read file due to exception: %ls", *FString(Exception.what()));
			return false;
		}

		// Check that the source grid indices are valid
		openvdb::GridPtrVecPtr Grids = Stream.getGrids();
		const size_t NumSourceGrids = Grids ? Grids->size() : 0;
		for (const FOpenVDBSparseVolumeAttributesDesc& AttributesDesc : Attributes)
		{
			for (const FOpenVDBSparseVolumeComponentMapping& Mapping : AttributesDesc.Mappings)
			{
				const int32 SourceGridIndex = Mapping.SourceGridIndex;
				if (SourceGridIndex != INDEX_NONE && SourceGridIndex >= (int32)NumSourceGrids)
				{
					UE_LOGF(LogSparseVolumeTextureOpenVDBUtility, Warning, "SparseVolumeTexture has invalid index into the array of grids in the source file: %i", SourceGridIndex);
					return false;
				}
			}
		}

		SVTCreateInfo.AttributesFormats[0] = MultiCompFormat[0];
		SVTCreateInfo.AttributesFormats[1] = MultiCompFormat[1];

		FIntVector3 SmallestAABBMin = FIntVector3(INT32_MAX);
		FIntVector3 LargestAABBMax = FIntVector3(INT32_MIN);
		TOptional<FTransform> PreviousGridTransform;

		// Compute per source grid data of up to 4 different grids (one per component)
		UniqueGridAdapters.SetNum((int32)NumSourceGrids);
		GridToComponentMappings.SetNum((int32)NumSourceGrids);
		for (int32 AttributesIdx = 0; AttributesIdx < NumAttributesDescs; ++AttributesIdx)
		{
			for (int32 CompIdx = 0; CompIdx < 4; ++CompIdx)
			{
				const uint32 SourceGridIndex = Attributes[AttributesIdx].Mappings[CompIdx].SourceGridIndex;
				const uint32 SourceComponentIndex = Attributes[AttributesIdx].Mappings[CompIdx].SourceComponentIndex;
				if (SourceGridIndex == INDEX_NONE)
				{
					continue;
				}

				openvdb::GridBase::Ptr GridBase = (*Grids)[SourceGridIndex];

				// Try to reuse adapters. Internally they use caching to accelerate read accesses, 
				// so using three different adapters to access the three components of a single grid would be wasteful.
				if (UniqueGridAdapters[SourceGridIndex] == nullptr)
				{
					UniqueGridAdapters[SourceGridIndex] = CreateOpenVDBGridAdapter(GridBase);
					if (!UniqueGridAdapters[SourceGridIndex])
					{
						return false;
					}
				}

				FOpenVDBGridInfo GridInfo = GetOpenVDBGridInfo(GridBase, 0, false);
				if (!IsOpenVDBGridValid(GridInfo, TEXT("")))
				{
					return false;
				}

				SmallestAABBMin.X = FMath::Min(SmallestAABBMin.X, GridInfo.VolumeActiveAABBMin.X);
				SmallestAABBMin.Y = FMath::Min(SmallestAABBMin.Y, GridInfo.VolumeActiveAABBMin.Y);
				SmallestAABBMin.Z = FMath::Min(SmallestAABBMin.Z, GridInfo.VolumeActiveAABBMin.Z);
				LargestAABBMax.X = FMath::Max(LargestAABBMax.X, GridInfo.VolumeActiveAABBMax.X);
				LargestAABBMax.Y = FMath::Max(LargestAABBMax.Y, GridInfo.VolumeActiveAABBMax.Y);
				LargestAABBMax.Z = FMath::Max(LargestAABBMax.Z, GridInfo.VolumeActiveAABBMax.Z);

				if (PreviousGridTransform && !PreviousGridTransform->Equals(GridInfo.Transform))
				{
					UE_LOGF(LogSparseVolumeTextureOpenVDBUtility, Warning, "Frame has multiple grids with different transforms in the same frame! Data will likely not be imported/displayed correctly!");
				}
				PreviousGridTransform = GridInfo.Transform;

				SVTCreateInfo.FallbackValues[AttributesIdx][CompIdx] = UniqueGridAdapters[SourceGridIndex]->GetBackgroundValue(SourceComponentIndex);

				FSingleGridToComponentMapping Mapping{};
				Mapping.AttributesIdx = (int32)AttributesIdx;
				Mapping.ComponentIdx = (int32)CompIdx;
				Mapping.GridComponentIdx = (int32)SourceComponentIndex;
				GridToComponentMappings[SourceGridIndex].Add(Mapping);
			}
		}

		const bool bEmptyBounds = SmallestAABBMin.X >= LargestAABBMax.X || SmallestAABBMin.Y >= LargestAABBMax.Y || SmallestAABBMin.Z >= LargestAABBMax.Z;
		SVTCreateInfo.VirtualVolumeAABBMin = bEmptyBounds ? FIntVector3(INT32_MAX) : (SmallestAABBMin - VolumeBoundsMin);
		SVTCreateInfo.VirtualVolumeAABBMax = bEmptyBounds ? FIntVector3(INT32_MIN) : (LargestAABBMax - VolumeBoundsMin);
		
		FrameTransform = PreviousGridTransform ? *PreviousGridTransform : FTransform::Identity;
		FrameTransform.AddToTranslation(FVector(VolumeBoundsMin) * FrameTransform.GetScale3D()); // We made the volume relative to VolumeBoundsMin, so we need to undo this translation when using the frame transform

		return true;
	}

	virtual UE::SVT::FTextureDataCreateInfo GetCreateInfo() const override
	{
		return SVTCreateInfo;
	}

	void IteratePhysicalSource(TFunctionRef<void(const FIntVector3& Coord, int32 AttributesIdx, int32 ComponentIdx, float VoxelValue)> OnVisit) const override
	{
		for (int32 GridIdx = 0; GridIdx < UniqueGridAdapters.Num(); ++GridIdx)
		{
			if (!UniqueGridAdapters[GridIdx])
			{
				continue;
			}

			UniqueGridAdapters[GridIdx]->IteratePhysical(
				[&](const FIntVector3& Coord, uint32 NumVoxelComponents, float* VoxelValues)
				{
					FIntVector3 RemappedCoord = Coord - VolumeBoundsMin;
					for (const FSingleGridToComponentMapping& Mapping : GridToComponentMappings[GridIdx])
					{
						OnVisit(RemappedCoord, Mapping.AttributesIdx, Mapping.ComponentIdx, VoxelValues[Mapping.GridComponentIdx]);
					}
				});
		}
	}

	FTransform GetFrameTransform() const { return FrameTransform; }

private:

	struct FSingleGridToComponentMapping
	{
		int32 AttributesIdx;
		int32 ComponentIdx;
		int32 GridComponentIdx;
	};

	static constexpr int32 NumAttributesDescs = 2;
	TArray<TSharedPtr<IOpenVDBGridAdapterBase>> UniqueGridAdapters;
	TArray<TArray<FSingleGridToComponentMapping, TInlineAllocator<4>>> GridToComponentMappings;
	TStaticArray<FOpenVDBSparseVolumeAttributesDesc, NumAttributesDescs> Attributes;
	TStaticArray<uint32, NumAttributesDescs> NumComponents;
	UE::SVT::FTextureDataCreateInfo SVTCreateInfo;
	FIntVector3 VolumeBoundsMin;
	FTransform FrameTransform;
};

#endif // OPENVDB_AVAILABLE

bool ConvertOpenVDBToSparseVolumeTexture(TArray64<uint8>& SourceFile, const FOpenVDBImportOptions& ImportOptions, const FIntVector3& VolumeBoundsMin, UE::SVT::FTextureData& OutResult, FTransform& OutFrameTransform)
{
#if OPENVDB_AVAILABLE
	FSparseVolumeTextureDataProviderOpenVDB DataProvider;
	if (!DataProvider.Initialize(SourceFile, ImportOptions, VolumeBoundsMin))
	{
		return false;
	}
	if (!OutResult.Create(DataProvider))
	{
		return false;
	}
	OutFrameTransform = DataProvider.GetFrameTransform();
	return true;
#else
	return false;
#endif // OPENVDB_AVAILABLE
}

const TCHAR* OpenVDBGridTypeToString(EOpenVDBGridType Type)
{
	switch (Type)
	{
	case EOpenVDBGridType::Half:
		return TEXT("Half");
	case EOpenVDBGridType::Half2:
		return TEXT("Half2");
	case EOpenVDBGridType::Half3:
		return TEXT("Half3");
	case EOpenVDBGridType::Half4:
		return TEXT("Half4");
	case EOpenVDBGridType::Float:
		return TEXT("Float");
	case EOpenVDBGridType::Float2:
		return TEXT("Float2");
	case EOpenVDBGridType::Float3:
		return TEXT("Float3");
	case EOpenVDBGridType::Float4:
		return TEXT("Float4");
	case EOpenVDBGridType::Double:
		return TEXT("Double");
	case EOpenVDBGridType::Double2:
		return TEXT("Double2");
	case EOpenVDBGridType::Double3:
		return TEXT("Double3");
	case EOpenVDBGridType::Double4:
		return TEXT("Double4");
	default:
		return TEXT("Unknown");
	}
}

FString GetVDBSequenceBaseFileName(const FString& FileName, bool bDiscardNumbersOnly)
{
	const FString CleanFileName = FPaths::GetCleanFilename(FileName);
	int32 NumValidChars = CleanFileName.Len() - 4; // chop off the file extension

	// Remove any digits at the end
	NumValidChars = CleanFileName.FindLastCharByPredicate([&](TCHAR Letter){ return !FChar::IsDigit(Letter); }, NumValidChars) + 1;

	// Optionally remove other unwanted chars like underscores and invalid object name chars that would later be replaced by underscores
	if (!bDiscardNumbersOnly)
	{
		NumValidChars = CleanFileName.FindLastCharByPredicate([&](TCHAR Letter)
			{
				// INVALID_OBJECTNAME_CHARACTERS is defined in NameTypes.h and is a string literal containing all the invalid chars.
				// The number at the end of a filename in a sequence is often separated by an underscore or other special character,
				// so when we want to get the base filename for deriving the new asset name, we also discard these characters.
				// Underscores are not part of the invalid chars string literal, so we append them here to also discard these chars.
				const TCHAR* InvalidChars = TEXT("_") INVALID_OBJECTNAME_CHARACTERS;
				while (*InvalidChars)
				{
					if (Letter == *InvalidChars)
					{
						return false;
					}
					++InvalidChars;
				}
				return true;
			}, NumValidChars) + 1;
	}

	const FString CleanFileNameWithoutSuffix = CleanFileName.Left(NumValidChars);
	return CleanFileNameWithoutSuffix;
}

TArray<FString> FindOpenVDBSequenceFileNames(const FString& Filename)
{
	TArray<FString> SequenceFilenames;

	// The file is potentially a sequence if the character before the `.vdb` is a number.
	const bool bIsFilePotentiallyPartOfASequence = FChar::IsDigit(Filename[Filename.Len() - 5]);

	if (!bIsFilePotentiallyPartOfASequence)
	{
		SequenceFilenames.Add(Filename);
	}
	else
	{
		const FString Path = FPaths::GetPath(Filename);
		const FString CleanFilename = FPaths::GetCleanFilename(Filename);
		const FString CleanFilenameWithoutSuffix = GetVDBSequenceBaseFileName(Filename, true /*bDiscardNumbersOnly*/);

		// Find all files potentially part of the sequence
		TArray<FString> PotentialSequenceFilenames;
		IFileManager::Get().FindFiles(PotentialSequenceFilenames, *Path, TEXT("*.vdb"));
		PotentialSequenceFilenames = PotentialSequenceFilenames.FilterByPredicate([&CleanFilenameWithoutSuffix](const FString& Str)
			{
				if (!CleanFilenameWithoutSuffix.IsEmpty())
				{
					// Check for the same base filename
					return Str.StartsWith(CleanFilenameWithoutSuffix);
				}
				else
				{
					// Removing the digits at the end of the input file resulted in an empty string, so we are looking for numeric filenames only (excluding the 4 chars file extension)
					return Str.LeftChop(4).IsNumeric();
				}
			});

		auto GetFilenameNumberSuffix = [](const FString& Filename) -> int32
		{
			const FString FilenameWithoutExt = Filename.LeftChop(4);
			const int32 LastNonDigitIndex = FilenameWithoutExt.FindLastCharByPredicate([](TCHAR Letter) { return !FChar::IsDigit(Letter); }) + 1;
			const FString NumberSuffixStr = FilenameWithoutExt.RightChop(LastNonDigitIndex);

			int32 Number = INDEX_NONE;
			if (NumberSuffixStr.IsNumeric())
			{
				TTypeFromString<int32>::FromString(Number, *NumberSuffixStr);
			}
			return Number;
		};

		// Find range of number suffixes
		int32 LowestIndex = INT32_MAX;
		int32 HighestIndex = INT32_MIN;
		for (FString& ItemFilename : PotentialSequenceFilenames)
		{
			const int32 Index = GetFilenameNumberSuffix(ItemFilename);
			if (Index == INDEX_NONE)
			{
				ItemFilename.Empty();
				continue;
			}
			LowestIndex = FMath::Min(LowestIndex, Index);
			HighestIndex = FMath::Max(HighestIndex, Index);
		}

		check(HighestIndex >= LowestIndex);

		// Sort the filenames into the result array
		SequenceFilenames.SetNum(HighestIndex - LowestIndex + 1);
		for (const FString& ItemFilename : PotentialSequenceFilenames)
		{
			const int32 Index = ItemFilename.IsEmpty() ? INDEX_NONE : GetFilenameNumberSuffix(ItemFilename);
			if (Index == INDEX_NONE)
			{
				continue;
			}
			SequenceFilenames[Index - LowestIndex] = Path / ItemFilename;
		}

		// Chop off any items after finding the first gap
		for (int32 i = 0; i < SequenceFilenames.Num(); ++i)
		{
			if (SequenceFilenames[i].IsEmpty())
			{
				SequenceFilenames.SetNum(i);
				break;
			}
		}
	}

	check(!SequenceFilenames.IsEmpty());

	return SequenceFilenames;
}

#endif // WITH_EDITOR