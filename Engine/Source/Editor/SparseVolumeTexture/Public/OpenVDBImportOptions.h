// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/StaticArray.h"

#include "OpenVDBImportOptions.generated.h"

enum class ESparseVolumeAttributesFormat : uint8
{
	Unorm8 = 0,
	Float16 = 1,
	Float32 = 2,
};

struct FOpenVDBSparseVolumeComponentMapping
{
	int32 SourceGridIndex = INDEX_NONE;
	int32 SourceComponentIndex = INDEX_NONE;
};

struct FOpenVDBSparseVolumeAttributesDesc
{
	TStaticArray<FOpenVDBSparseVolumeComponentMapping, 4> Mappings;
	ESparseVolumeAttributesFormat Format = ESparseVolumeAttributesFormat::Unorm8;
};

struct FOpenVDBImportOptions
{
	TStaticArray<FOpenVDBSparseVolumeAttributesDesc, 2> Attributes;
	bool bIsSequence = false;
};

enum class EOpenVDBGridType : uint8
{
	Unknown = 0,
	Half,
	Half2,
	Half3,
	Half4,
	Float,
	Float2,
	Float3,
	Float4,
	Double,
	Double2,
	Double3,
	Double4,
};

struct FOpenVDBGridInfo
{
	FTransform Transform;
	FIntVector3 VolumeActiveAABBMin;
	FIntVector3 VolumeActiveAABBMax;
	FIntVector3 VolumeActiveDim;
	FString Name;
	FString DisplayString; // Contains Index (into source file grids), Type and Name
	uint32 Index;
	uint32 NumComponents;
	EOpenVDBGridType Type;
	bool bIsInWorldSpace;
};

struct FOpenVDBGridComponentInfo
{
	uint32 Index;
	uint32 ComponentIndex;
	FString Name;
	FString DisplayString; // Contains source file grid index, name and component (if it is a multi component type like Float3)
};

struct FOpenVDBPreviewData
{
	TArray64<uint8> LoadedFile;
	TArray<FOpenVDBGridInfo> GridInfo;
	TArray<TSharedPtr<FOpenVDBGridInfo>> GridInfoPtrs;
	TArray<TSharedPtr<FOpenVDBGridComponentInfo>> GridComponentInfoPtrs;
	TArray<FString> SequenceFilenames;
	FOpenVDBImportOptions DefaultImportOptions;
};

UCLASS(MinimalAPI)
class UOpenVDBImportOptionsObject : public UObject
{
	GENERATED_BODY()

public:
	FOpenVDBPreviewData PreviewData;
};
