// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "EditorReimportHandler.h"

#include "SparseVolumeTextureFactory.generated.h"

struct FOpenVDBPreviewData;

// Responsible for creating and importing Sparse Volume Texture objects
UCLASS(hidecategories = Object, MinimalAPI)
class USparseVolumeTextureFactory : public UFactory, public FReimportHandler
{
	GENERATED_UCLASS_BODY()

public:

	//~ Begin UFactory Interface

	virtual bool ShouldShowInNewMenu() const override;
	virtual FText GetDisplayName() const override;
	virtual bool ConfigureProperties() override;

	///////////////////////////////////////////////////////////////////////////////
	// Create asset

	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;

	virtual bool CanCreateNew() const override;

	///////////////////////////////////////////////////////////////////////////////
	// Import asset

	virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename,
		const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;

	virtual void CleanUp() override;
	virtual bool FactoryCanImport(const FString& Filename) override;
	virtual bool DoesSupportClass(UClass* Class) override;
	virtual UClass* ResolveSupportedClass() override;

	//~ End UFactory Interface

	//~ Begin FReimportHandler Interface
	virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
	virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
	virtual EReimportResult::Type Reimport(UObject* Obj) override;
	//~ End FReimportHandler Interface

	SPARSEVOLUMETEXTURE_API UObject* ImportWithTransform(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename,
		const TCHAR* Parms, bool& bOutOperationCanceled, const FTransform& AdditionalFrameTransform);

protected:

private:
	UObject* ImportInternal(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, bool& bOutOperationCanceled, bool bIsReimport, const FTransform& AdditionalFrameTransform = FTransform::Identity);
};

bool SPARSEVOLUMETEXTURE_API LoadOpenVDBPreviewData(const FString& Filename, FOpenVDBPreviewData* OutPreviewData);

#endif // WITH_EDITOR
