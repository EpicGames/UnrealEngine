// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "IAssetTools.h"
#include "AssetRegistry/AssetRegistryModule.h"

/**
 * A sparse volume texture
 */
class FSparseVolumeTextureModule : public IModuleInterface
{
public:

	static FSparseVolumeTextureModule& GetSparseVolumeTextureModule()
	{
		static const FName ModuleName = TEXT("SparseVolumeTexture");
		auto& ModuleInterface = FModuleManager::LoadModuleChecked<FSparseVolumeTextureModule>(ModuleName);
		return ModuleInterface;
	}

private:

	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule();

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule();
};
