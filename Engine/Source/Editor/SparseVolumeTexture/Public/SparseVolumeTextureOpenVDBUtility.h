// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"

namespace UE
{
	namespace SVT
	{
		struct FTextureData;
	}
}
struct FOpenVDBGridInfo;
enum class EOpenVDBGridType : uint8;

struct FOpenVDBToSVTConversionResult
{
	struct FSparseVolumeAssetHeader* Header;
	TArray<uint32>* PageTable;
	TArray<uint8>* PhysicalTileDataA;
	TArray<uint8>* PhysicalTileDataB;
};

bool IsOpenVDBGridValid(const FOpenVDBGridInfo& GridInfo, const FString& Filename);

bool SPARSEVOLUMETEXTURE_API GetOpenVDBGridInfo(TArray64<uint8>& SourceFile, bool bCreateStrings, TArray<FOpenVDBGridInfo>* OutGridInfo);

bool SPARSEVOLUMETEXTURE_API ConvertOpenVDBToSparseVolumeTexture(TArray64<uint8>& SourceFile, const struct FOpenVDBImportOptions& ImportOptions, const FIntVector3& VolumeBoundsMin, UE::SVT::FTextureData& OutResult, FTransform& OutFrameTransform);

const TCHAR* OpenVDBGridTypeToString(EOpenVDBGridType Type);

FString SPARSEVOLUMETEXTURE_API GetVDBSequenceBaseFileName(const FString& FileName, bool bDiscardNumbersOnly);

TArray<FString> SPARSEVOLUMETEXTURE_API FindOpenVDBSequenceFileNames(const FString& Filename);

#endif // WITH_EDITOR