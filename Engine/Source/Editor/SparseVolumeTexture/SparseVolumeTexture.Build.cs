// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class SparseVolumeTexture : ModuleRules
{
	public SparseVolumeTexture(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Projects",
				"Engine",
				"Renderer",
				"RenderCore",
				"Slate",
				"SlateCore",
				"ApplicationCore",
				"ToolWidgets",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"AssetRegistry",
				"AssetTools",
				"EditorWidgets",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"RHI",
				"CoreUObject",
				"InputCore",
				"Settings",
			}
		);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"MainFrame",
					"EditorFramework",
					"UnrealEd"
				}
			);
		}

		bDisableAutoRTFMInstrumentation = true;

		// Specific to OpenVDB support
		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows))
		{
			bUseRTTI = true;
			bEnableExceptions = true;
			PublicDefinitions.Add("OPENVDB_AVAILABLE=1");

			AddEngineThirdPartyPrivateStaticDependencies(Target,
				"IntelTBB",
				"Blosc",
				"zlib",
				"Boost",
				"OpenVDB"
			);
		}
		else if (Target.Platform == UnrealTargetPlatform.Linux)
		{
			bUseRTTI = false;
			bEnableExceptions = true;
			PublicDefinitions.Add("OPENVDB_AVAILABLE=1");

			AddEngineThirdPartyPrivateStaticDependencies(Target,
				"IntelTBB",
				"Blosc",
				"zlib",
				"Boost",
				"OpenVDB"
			);
		}
		else
		{
			PublicDefinitions.Add("OPENVDB_AVAILABLE=0");
		}
	}
}
